[CoreRedirects]

;
; 4.26 Redirects
;

; HitchDetection Settings
+PropertyRedirects=(OldName="StageHitchDetectionSettings.TargetFrameRate",NewName="StageHitchDetectionSettings.MinimumFrameRate")

; 5.3 Redirects
;
+PropertyRedirects=(OldName="FramePerformanceProviderMessage.ShadersToCompile",NewName="FramePerformanceProviderMessage.CompilationTasksRemaining")
