// Copyright Epic Games, Inc. All Rights Reserved.

#include "FramePerformanceProvider.h"

#include "AssetCompilingManager.h"
#include "EngineGlobals.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "IStageDataProvider.h"
#include "Misc/App.h"
#include "RHI.h"
#include "RenderTimer.h"

#include "StageDataProviderModule.h"
#include "StageMonitorUtils.h"
#include "Stats/StatsData.h"
#include "UObject/Package.h"
#include "UObject/PackageReload.h"

namespace UE::FramePerformanceProvider::Private
{

int32 CompilationTasksRemaining()
{
	FAssetCompilingManager& Manager = FAssetCompilingManager::Get();
	return Manager.GetNumRemainingAssets();
}

#if STATS
uint64 TotalGPUResourceSize(const TArray<FStatMessage>& StatMessages)
{
	uint64 TotalSize = 0;
	FName NAME_STATGROUP_RHI(FStatGroup_STATGROUP_RHI::GetGroupName());
	for (const FStatMessage& Stat : StatMessages)
	{
		FName LastGroup = Stat.NameAndInfo.GetGroupName();
		if (LastGroup == NAME_STATGROUP_RHI && Stat.NameAndInfo.GetFlag(EStatMetaFlags::IsMemory))
		{
			TotalSize += Stat.GetValue_int64();
		}
	}
	return TotalSize;
}
#endif // STATS

static float GAverageGPU = 0;

void UpdateAverageGPUUsage()
{
	const float GPUTime = FPlatformTime::ToMilliseconds(RHIGetGPUFrameCycles());
	GAverageGPU = 0.75*GAverageGPU + 0.25*GPUTime;
}

#if STATS
FFramePerformanceProviderMessage GetLatestPerformanceData(const	TArray<FStatMessage>& StatMessages)
#else
FFramePerformanceProviderMessage GetLatestPerformanceData()
#endif // STATS
{
	const float GameThreadTime = FPlatformTime::ToMilliseconds(GGameThreadTime);
	const float GameThreadWaitTime = FPlatformTime::ToMilliseconds(GGameThreadWaitTime);
	const float RenderThreadTime = FPlatformTime::ToMilliseconds(GRenderThreadTime);
	const float RenderThreadWaitTime = FPlatformTime::ToMilliseconds(GRenderThreadWaitTime);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const float IdleTimeMilli = (FAppTime::Get_UnsafeNoCheck().IdleTime * 1000.0);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if STATS
	FPlatformMemoryStats Stats = FPlatformMemory::GetStats();
	const uint64 CPUMem = Stats.UsedPhysical;
	const uint64 GPUMem = TotalGPUResourceSize(StatMessages);
#else
	constexpr uint64 CPUMem = 0;
	constexpr uint64 GPUMem = 0;
#endif // STATS

	return FFramePerformanceProviderMessage(
		EStageMonitorNodeStatus::Unknown,
		GameThreadTime,
		GameThreadWaitTime,
		RenderThreadTime,
		RenderThreadWaitTime,
		GAverageGPU,
		IdleTimeMilli,
		CPUMem,
		GPUMem,
		CompilationTasksRemaining()
	);
}

}

class FFrameProviderRunner : public FRunnable
{
public:
	FFrameProviderRunner()
	{
		Thread.Reset(FRunnableThread::Create( this, TEXT( "StageMonitor Frame Provider Thread" ) ));
	}

	virtual ~FFrameProviderRunner()
	{
		if (Thread)
		{
			Thread->Kill();
		}
	}

	// FRunnable functions
	virtual uint32 Run() override
	{
#if STATS
		double LastRHIUpdate = FPlatformTime::Seconds();
#endif // STATS

		bStopped = false;
		do
		{
#if STATS
			const double CurrentPlatformTimeInSeconds = FPlatformTime::Seconds();
			if (CurrentPlatformTimeInSeconds > (UpdateRHIResourcesFrequency + LastRHIUpdate)
				|| Stats.IsEmpty())
			{
				Stats.Reset();
				LastRHIUpdate = CurrentPlatformTimeInSeconds;
				GetPermanentStats(Stats);
			}
#endif // STATS

			UE::FramePerformanceProvider::Private::UpdateAverageGPUUsage();
			IStageDataProvider::SendMessage<FFramePerformanceProviderMessage>(EStageMessageFlags::None, GetFramePerformanceData());

			FPlatformProcess::Sleep(UpdateFrequency);
		} while (!bStopped);
		return 0;
	}

	virtual void Stop() override
	{
		bStopped = true;
	}

	virtual void Exit() override
	{
		Stop();
	}

	// FRunnable

	struct FLoadInfo
	{
		EStageMonitorNodeStatus Status;
		FString AssetName;
	};

	FFramePerformanceProviderMessage GetFramePerformanceData()
	{
#if STATS
		FFramePerformanceProviderMessage OutboundData = UE::FramePerformanceProvider::Private::GetLatestPerformanceData(Stats);
#else
		FFramePerformanceProviderMessage OutboundData = UE::FramePerformanceProvider::Private::GetLatestPerformanceData();
#endif // STATS

		if (OutboundData.CompilationTasksRemaining > 0)
		{
			OutboundData.Status = EStageMonitorNodeStatus::AssetCompiling;
		}
		else
		{
			TOptional<FLoadInfo> LatestLoadInfo = GetLoadInfoTop();
			if (LatestLoadInfo)
			{
				OutboundData.Status = LatestLoadInfo->Status;
				OutboundData.AssetInStatus = LatestLoadInfo->AssetName;
			}
			else
			{
				OutboundData.Status = EStageMonitorNodeStatus::Ready;
			}
		}

		return OutboundData;
	}

	void PushLoadInfo(FLoadInfo Info)
	{
		FScopeLock Lock(&LoadInfoCS);
		LoadStack.Push(MoveTemp(Info));
	}

	TOptional<FLoadInfo> GetLoadInfoTop() const
	{
		FScopeLock Lock(&LoadInfoCS);
		if (LoadStack.Num() > 0)
		{
			return LoadStack.Top();
		}
		return {};
	}

	TOptional<FLoadInfo> PopLoadInfo()
	{
		FScopeLock Lock(&LoadInfoCS);
		if (LoadStack.Num() > 0)
		{
			TOptional<FLoadInfo> Info = LoadStack.Top();
			LoadStack.Pop();
			return Info;
		}
		return {};
	}

	void SetUpdateFrequency(float InFrequency)
	{
		UpdateFrequency = InFrequency;
	}

	TUniquePtr<FRunnableThread> Thread;
	mutable FCriticalSection LoadInfoCS;

	TArray<FLoadInfo> LoadStack;

#if STATS
	TArray<FStatMessage> Stats;
#endif // STATS

	const float UpdateRHIResourcesFrequency = 4.0f;
	float UpdateFrequency = 0.2f; // Default is 200ms;
	bool bStopped = false;
};


FFramePerformanceProvider::FFramePerformanceProvider()
{
	FCoreUObjectDelegates::PreLoadMap.AddRaw(this, &FFramePerformanceProvider::HandlePreLoadMap);
	FCoreUObjectDelegates::PostLoadMapWithWorld.AddRaw(this, &FFramePerformanceProvider::HandlePostLoadMap);
	FCoreUObjectDelegates::OnPackageReloaded.AddRaw(this, &FFramePerformanceProvider::HandleAssetReload);

	ProviderThread = MakeUnique<FFrameProviderRunner>();
#if WITH_EDITOR
	GetMutableDefault<UStageMonitoringSettings>()->OnSettingChanged().AddRaw(this, &FFramePerformanceProvider::OnStageSettingsChanged);

	const double UpdateInterval = GetDefault<UStageMonitoringSettings>()->ProviderSettings.FramePerformanceSettings.UpdateInterval;
	ProviderThread->SetUpdateFrequency(static_cast<float>(UpdateInterval));
#endif //WITH_EDITOR
	EnableHitchDetection(GetDefault<UStageMonitoringSettings>()->ProviderSettings.HitchDetectionSettings.bEnableHitchDetection);
}

FFramePerformanceProvider::~FFramePerformanceProvider()
{
	FCoreUObjectDelegates::PreLoadMap.RemoveAll(this);
	FCoreUObjectDelegates::PostLoadMapWithWorld.RemoveAll(this);
	FCoreUObjectDelegates::OnPackageReloaded.RemoveAll(this);

	EnableHitchDetection(false);
}

/** Delegate for when package reload occurs. */
void FFramePerformanceProvider::HandleAssetReload(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent)
{
	if (InPackageReloadPhase == EPackageReloadPhase::PrePackageFixup)
	{
		FString Name = InPackageReloadedEvent->GetOldPackage()->GetFullName();
		ProviderThread->PushLoadInfo({EStageMonitorNodeStatus::HotReload, Name});
		IStageDataProvider::SendMessage<FAssetLoadingStateProviderMessage>(
			EStageMessageFlags::Reliable, EStageLoadingState::PreLoad, Name);
	}
	else if (InPackageReloadPhase == EPackageReloadPhase::PostPackageFixup)
	{
		TOptional<FFrameProviderRunner::FLoadInfo> Info = ProviderThread->PopLoadInfo();

		if (ensure(Info))
		{
			IStageDataProvider::SendMessage<FAssetLoadingStateProviderMessage>(
				EStageMessageFlags::Reliable, EStageLoadingState::PostLoad, Info->AssetName);
		}
	}
}

/** Delegate for pre-load map */
void FFramePerformanceProvider::HandlePreLoadMap(const FString& MapName)
{
	ProviderThread->PushLoadInfo({EStageMonitorNodeStatus::LoadingMap, MapName});
	IStageDataProvider::SendMessage<FAssetLoadingStateProviderMessage>(
		EStageMessageFlags::Reliable, EStageLoadingState::PreLoad, MapName);
}

/** Delegate for post load of map. */
void FFramePerformanceProvider::HandlePostLoadMap(UWorld* /*unused*/)
{
	TOptional<FFrameProviderRunner::FLoadInfo> Info = ProviderThread->PopLoadInfo();

	if (ensure(Info))
	{
		IStageDataProvider::SendMessage<FAssetLoadingStateProviderMessage>(
			EStageMessageFlags::Reliable, EStageLoadingState::PostLoad, Info->AssetName);
	}
}

void FFramePerformanceProvider::CheckHitches(int64 Frame)
{
#if STATS
	// when synced, this time will be the full time of the frame, whereas the above don't include any waits
	const FStatsThreadState& Stats = FStatsThreadState::GetLocalState();
	const float GameThreadTimeWithWaits = (float)FPlatformTime::ToMilliseconds64(Stats.GetFastThreadFrameTime(Frame, EThreadType::Game));
	const float RenderThreadTimeWithWaits = (float)FPlatformTime::ToMilliseconds64(Stats.GetFastThreadFrameTime(Frame, EThreadType::Renderer));
	const float FullFrameTime = FMath::Max(GameThreadTimeWithWaits, RenderThreadTimeWithWaits);

	// check for hitch (if application not backgrounded)
	const float TimeThreshold = CachedHitchSettings.MinimumFrameRate.AsInterval() * 1000.0f;
	if (FullFrameTime > TimeThreshold)
	{
		const float GameThreadTime = FPlatformTime::ToMilliseconds(GGameThreadTime);
		const float RenderThreadTime = FPlatformTime::ToMilliseconds(GRenderThreadTime);
		const float GPUTime = FPlatformTime::ToMilliseconds(RHIGetGPUFrameCycles());
		float HitchedFPS = CachedHitchSettings.MinimumFrameRate.AsDecimal();
		if (!FMath::IsNearlyZero(FullFrameTime))
		{
			HitchedFPS = 1000.0f / FullFrameTime;
		}

		UE_LOGF(LogStageDataProvider, VeryVerbose, "Hitch detected: FullFrameTime=%f, GameThreadTimeWithWaits=%f, RenderThreadTimeWithWaits=%f, Threshold=%f, GameThreadTime=%f, RenderThreadTime=%f", FullFrameTime, GameThreadTimeWithWaits, RenderThreadTimeWithWaits, TimeThreshold, GameThreadTime, RenderThreadTime);

		IStageDataProvider::SendMessage<FHitchDetectionMessage>(EStageMessageFlags::None, GameThreadTimeWithWaits, RenderThreadTimeWithWaits, GameThreadTime, RenderThreadTime, GPUTime, TimeThreshold, HitchedFPS);
	}
#endif //STATS
}

#if WITH_EDITOR
void FFramePerformanceProvider::OnStageSettingsChanged(UObject* Object, struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
	{
		const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

		if (PropertyName == GET_MEMBER_NAME_CHECKED(FStageHitchDetectionSettings, bEnableHitchDetection))
		{
			EnableHitchDetection(GetDefault<UStageMonitoringSettings>()->ProviderSettings.HitchDetectionSettings.bEnableHitchDetection);
		}
		if (PropertyName == GET_MEMBER_NAME_CHECKED(FStageFramePerformanceSettings, UpdateInterval))
		{
			const double UpdateInterval = GetDefault<UStageMonitoringSettings>()->ProviderSettings.FramePerformanceSettings.UpdateInterval;
			ProviderThread->SetUpdateFrequency(static_cast<float>(UpdateInterval));
		}
	}
}

#endif //WITH_EDITOR

void FFramePerformanceProvider::EnableHitchDetection(bool bShouldEnable)
{
#if STATS
	if (bShouldEnable != bIsHitchDetectionEnabled)
	{
		if (bShouldEnable)
		{
			CachedHitchSettings = GetDefault<UStageMonitoringSettings>()->ProviderSettings.HitchDetectionSettings;

			// Subscribe to Stats provider to verify hitches
			StatsPrimaryEnableAdd();
			FStatsThreadState& Stats = FStatsThreadState::GetLocalState();
			Stats.NewFrameDelegate.AddRaw(this, &FFramePerformanceProvider::CheckHitches);
		}
		else
		{
			StatsPrimaryEnableSubtract();
			FStatsThreadState& Stats = FStatsThreadState::GetLocalState();
			Stats.NewFrameDelegate.RemoveAll(this);
		}
	}

	bIsHitchDetectionEnabled = bShouldEnable;
#endif //STATS
}
