// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "StageMessages.h"
#include "StageMonitoringSettings.h"


#include "FramePerformanceProvider.generated.h"

class FFrameProviderRunner;

/**
 * Message sent when a hitch was detected on a provider machine.
 */
USTRUCT()
struct STAGEDATAPROVIDER_API FHitchDetectionMessage : public FStageProviderEventMessage
{
	GENERATED_BODY()

public:
	FHitchDetectionMessage() = default;

	FHitchDetectionMessage(float InGameThreadTimeWithWaits, float InRenderThreadTimeWithWaits, float InGameThreadTime, float InRenderThreadTime, float InGPUTime, float InTimingThreshold, float InHitchedFPS)
		: GameThreadWithWaitsMS(InGameThreadTimeWithWaits)
		, RenderThreadWithWaitsMS(InRenderThreadTimeWithWaits)
		, GameThreadMS(InGameThreadTime)
		, RenderThreadMS(InRenderThreadTime)
		, GPU_MS(InGPUTime)
		, TimingThreshold(InTimingThreshold)
		, HitchedTimeFPS(InHitchedFPS)
	{
		extern ENGINE_API float GAverageFPS;
		AverageFPS = GAverageFPS;
	}

	/** Current GameThread time including any waits read from StatsThread in milliseconds */
	UPROPERTY(VisibleAnywhere, Category = "Hitch", meta = (Unit = "ms"))
	float GameThreadWithWaitsMS = 0.f;

	/** Current RenderThread time including any waits read from StatsThread in milliseconds */
	UPROPERTY(VisibleAnywhere, Category = "Hitch", meta = (Unit = "ms"))
	float RenderThreadWithWaitsMS = 0.f;

	/** Current GameThread time read from GGameThreadTime in milliseconds */
	UPROPERTY(VisibleAnywhere, Category = "Hitch", meta = (Unit = "ms"))
	float GameThreadMS = 0.f;

	/** Current RenderThread time read from GRenderThreadTime in milliseconds */
	UPROPERTY(VisibleAnywhere, Category = "Hitch", meta = (Unit = "ms"))
	float RenderThreadMS = 0.f;

	/** Current GPU time read from RHIGetGPUFrameCycles() in milliseconds */
	UPROPERTY(VisibleAnywhere, Category = "Hitch", meta = (Unit = "ms"))
	float GPU_MS = 0.f;

	/** Timing threshold that was crossed in milliseconds */
	UPROPERTY(VisibleAnywhere, Category = "Hitch", meta = (Unit = "ms"))
	float TimingThreshold = 0.f;

	/** FPS correspondent to the timing that triggered the hitch (game or render thread) */
	UPROPERTY(VisibleAnywhere, Category = "Hitch")
	float HitchedTimeFPS = 0.f;

	/** Average FPS when hitch occured */
	UPROPERTY(VisibleAnywhere, Category = "Hitch")
	float AverageFPS = 0.f;
};

/**
 * Handles sending frame performance messages periodically and hitch messages if detected
 * Either one can be enabled / disabled using project settings
 */
class FFramePerformanceProvider
{
public:
	FFramePerformanceProvider();
	~FFramePerformanceProvider();

private:

	/** Callback by the stat module to detect if a hitch happened */
	void CheckHitches(int64 Frame);

	/** Triggered when a stage monitor settings has changed */
#if WITH_EDITOR
	void OnStageSettingsChanged(UObject* Object, struct FPropertyChangedEvent& PropertyChangedEvent);
#endif //WITH_EDITOR

	/** Enables/disables hitch detection */
	void EnableHitchDetection(bool bShouldEnable);

private:

	/** Delegate for when package reload occurs. */
	void HandleAssetReload(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent);

	/** Delegate for pre-load map */
	void HandlePreLoadMap(const FString& MapName);

	/** Delegate for post load of map. */
	void HandlePostLoadMap(UWorld* /*unused*/);

	/** Cached settings for hitch detection since it's called on another thread, we can't access UObject */
	FStageHitchDetectionSettings CachedHitchSettings;

	bool bIsHitchDetectionEnabled = false;

	TUniquePtr<FFrameProviderRunner> ProviderThread;
};
