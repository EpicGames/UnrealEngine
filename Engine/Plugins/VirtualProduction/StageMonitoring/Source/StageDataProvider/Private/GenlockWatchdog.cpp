// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenlockWatchdog.h"

#include "Engine/Engine.h"
#include "GenlockedCustomTimeStep.h"
#include "IStageDataProvider.h"
#include "Misc/CoreDelegates.h"

FGenlockWatchdog::FGenlockWatchdog()
{
	FCoreDelegates::OnEndFrame.AddRaw(this, &FGenlockWatchdog::OnEndFrame);
}

FGenlockWatchdog::~FGenlockWatchdog()
{
	FCoreDelegates::OnEndFrame.RemoveAll(this);
}

void FGenlockWatchdog::OnEndFrame()
{
	if (GEngine)
	{
		UGenlockedCustomTimeStep* Genlock = Cast<UGenlockedCustomTimeStep>(GEngine->GetCustomTimeStep());
		if (Genlock)
		{
			const ECustomTimeStepSynchronizationState State = Genlock->GetSynchronizationState();
			if (State != LastState)
			{
				IStageDataProvider::SendMessage<FGenlockStateEvent>(EStageMessageFlags::Reliable, State);
				LastState = State;
			}

			if (State == ECustomTimeStepSynchronizationState::Synchronized &&  Genlock->IsLastSyncDataValid())
			{
				//Detects missed sync signals if genlock considered synchronized
				const int32 LastSyncCounts = Genlock->GetLastSyncCountDelta();
				if (LastSyncCounts != Genlock->GetExpectedSyncCountDelta())
				{
					IStageDataProvider::SendMessage<FGenlockHitchEvent>(EStageMessageFlags::Reliable, LastSyncCounts - Genlock->GetExpectedSyncCountDelta());
				}
			}
		}
		else
		{
			//If genlock ever became operational, add event for it now being closed
			if (LastState == ECustomTimeStepSynchronizationState::Synchronized)
			{
				LastState = ECustomTimeStepSynchronizationState::Closed;
				IStageDataProvider::SendMessage<FGenlockStateEvent>(EStageMessageFlags::Reliable, LastState);
			}
		}
	}
}

FString FGenlockHitchEvent::ToString() const
{
	return FString::Printf(TEXT("Missed %d sync signals"), MissedSyncSignals);
}

FString FGenlockStateEvent::ToString() const
{
	return FString::Printf(TEXT("Genlock state: %s"), *StaticEnum<ECustomTimeStepSynchronizationState>()->GetNameStringByValue((int64)NewState));
}
