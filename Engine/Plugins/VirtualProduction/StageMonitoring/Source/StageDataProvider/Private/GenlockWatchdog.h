// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Engine/EngineCustomTimeStep.h"
#include "StageMessages.h"

#include "GenlockWatchdog.generated.h"


/**
 * Stage event to notify of missed sync (genlock) signals
 * When sync signal are lost, it means the engine is not running 
 * fast enough to keep track of the genlock source. 
 */
USTRUCT()
struct FGenlockHitchEvent : public FStageProviderEventMessage
{
	GENERATED_BODY()

public:
	FGenlockHitchEvent() = default;
	FGenlockHitchEvent(int32 InMissedSyncs)
		: MissedSyncSignals(InMissedSyncs)
		{}

	virtual FString ToString() const override;

	/** Number of sync counts that were missed between tick */
	UPROPERTY(VisibleAnywhere, Category="GenlockState")
	int32 MissedSyncSignals = 0;
};

/**
 * Stage event to notify of genlock custom timestep state change
 */
USTRUCT()
struct FGenlockStateEvent : public FStageProviderEventMessage
{
	GENERATED_BODY()

public:
	FGenlockStateEvent() = default;
	FGenlockStateEvent(ECustomTimeStepSynchronizationState InState)
		: NewState(InState)
		{}

	virtual FString ToString() const override;

	/** New state of genlock custom timestep (i.e. Synchronized, Error, etc...) */
	UPROPERTY(VisibleAnywhere, Category = "GenlockState")
	ECustomTimeStepSynchronizationState NewState = ECustomTimeStepSynchronizationState::Closed;
};

/**
 * Verifies the state of the genlock and missed sync events if any and send stage event if required
 */
class FGenlockWatchdog 
{
public:
	FGenlockWatchdog();
	~FGenlockWatchdog();

private:

	/** At end of frame, verify genlock signal / state and notify state monitor if something changed */
	void OnEndFrame();

private:
	
	/** Previous value of the genlock state. */
	ECustomTimeStepSynchronizationState LastState = ECustomTimeStepSynchronizationState::Closed; 
};
