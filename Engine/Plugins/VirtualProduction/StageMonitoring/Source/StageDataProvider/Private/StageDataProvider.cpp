// Copyright Epic Games, Inc. All Rights Reserved.

#include "StageDataProvider.h"

#include "Engine/Engine.h"
#include "MessageEndpointBuilder.h"
#include "Misc/App.h"
#include "StageDataProviderModule.h"
#include "StageMonitorUtils.h"
#include "StageMonitoringSettings.h"
#include "VPRolesSubsystem.h"


namespace StageDataProviderUtils
{
	//Stage messaging version we support
	static const int32 SupportedStageMessageVersion = 1;
}


FStageDataProvider::FStageDataProvider()
	: Identifier(FGuid::NewGuid())
{

}

FStageDataProvider::~FStageDataProvider()
{
	Stop();
}

bool FStageDataProvider::Tick(float DeltaTime)
{
	RemoveTimeoutedMonitors();

	return true;
}

bool FStageDataProvider::SendMessageInternal(FStageDataBaseMessage* Payload, UScriptStruct* Type, EStageMessageFlags InFlags)
{
	if (IsMessageTypeExcluded(Type))
	{
		return false;
	}

	if(Monitors.Num() > 0)
	{
		check(Payload);

		//Message bus requires payload to be newed and life cycle will be taken care of in MB
		FStageDataBaseMessage* MessageData = reinterpret_cast<FStageDataBaseMessage*>(FMemory::Malloc(Type->GetStructureSize()));
		Type->InitializeStruct(MessageData);
		Type->CopyScriptStruct(MessageData, Payload);

		//Setup this provider identifier in sent message
		MessageData->Identifier = Identifier;

		//Send messages to connected monitors
		TArray<FMessageAddress> Addresses;
		for (const FMonitorEndpoints& Monitor : Monitors)
		{
			Addresses.Add(Monitor.Address);
		}

		const EMessageFlags Flags = EnumHasAnyFlags(InFlags, EStageMessageFlags::Reliable) ? EMessageFlags::Reliable : EMessageFlags::None;
		ProviderEndpoint->Send(MessageData, Type, Flags, nullptr, Addresses, FTimespan::Zero(), FDateTime::MaxValue());

		return true;
	}

	return false;
}

void FStageDataProvider::Start()
{
	if (!bIsActive)
	{
		ProviderEndpoint = FMessageEndpoint::Builder(TEXT("StageDataProvider"))
			.ReceivingOnThread(ENamedThreads::Type::GameThread)
			.Handling<FStageProviderDiscoveryMessage>(this, &FStageDataProvider::HandleDiscoveryMessage)
			.Handling<FStageMonitorCloseMessage>(this, &FStageDataProvider::HandleMonitorCloseMessage);

		if (ProviderEndpoint.IsValid())
		{
			ProviderEndpoint->Subscribe<FStageProviderDiscoveryMessage>();
			TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FStageDataProvider::Tick), 0.0f);

			bIsActive = true;
		}
	}
}

void FStageDataProvider::Stop()
{
	if (bIsActive)
	{
		//Tell monitors we're leaving
		SendMessage<FStageProviderCloseMessage>(EStageMessageFlags::Reliable);

		if (TickerHandle.IsValid())
		{
			FTSTicker::RemoveTicker(TickerHandle);
		}

		if (ProviderEndpoint.IsValid())
		{
			ProviderEndpoint->Disable();
			ProviderEndpoint.Reset();
		}

		Monitors.Empty();
		InvalidMonitors.Empty();

		bIsActive = false;
	}
}

void FStageDataProvider::HandleDiscoveryMessage(const FStageProviderDiscoveryMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (Message.StageMessageVersion != StageDataProviderUtils::SupportedStageMessageVersion)
	{
		if (!InvalidMonitors.Contains(Message.Identifier))
		{
			InvalidMonitors.Add(Message.Identifier);
			
			UE_LOGF(LogStageDataProvider, VeryVerbose, "Received discovery message from monitor (%ls:%d) with unsupported version '%d'. Supported version is '%d'"
				, *Message.Descriptor.MachineName
				, Message.Descriptor.ProcessId
				, Message.StageMessageVersion
				, StageDataProviderUtils::SupportedStageMessageVersion);
		}

		return;
	}

	const int32 Index = Monitors.IndexOfByPredicate([Message](const FMonitorEndpoints& Other) { return Other.Identifier == Message.Identifier; });
	if (Index == INDEX_NONE)
	{
		UE_LOGF(LogStageDataProvider, Log, "New monitor linked, %d", Message.Descriptor.ProcessId);

		FMonitorEndpoints NewMonitor;
		NewMonitor.Address = Context->GetSender();
		NewMonitor.Identifier = Message.Identifier;
		NewMonitor.Descriptor = Message.Descriptor;
		NewMonitor.LastReceivedMessageTime = FApp::GetCurrentTime();
		Monitors.Emplace(MoveTemp(NewMonitor));

		//Discovery response will contain detailed info about this provider
		FStageInstanceDescriptor Desc = StageMonitorUtils::GetInstanceDescriptor();
		SendMessage<FStageProviderDiscoveryResponseMessage>(EStageMessageFlags::None, MoveTemp(Desc));
	}
	else
	{
		//If we already have that monitor linked, just update its received time to avoid timeouts
		Monitors[Index].LastReceivedMessageTime = FApp::GetCurrentTime();
	}
}

void FStageDataProvider::HandleMonitorCloseMessage(const FStageMonitorCloseMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	const FGuid ProviderIdentifier = Message.Identifier;
	const int32 Index = Monitors.IndexOfByPredicate([ProviderIdentifier](const FMonitorEndpoints& Other) { return Other.Identifier == ProviderIdentifier; });
	if(Index != INDEX_NONE)
	{
		Monitors.RemoveAtSwap(Index);
	}
}

void FStageDataProvider::RemoveTimeoutedMonitors()
{
	for (auto It = Monitors.CreateIterator(); It; ++It)
	{
		const double DeltaTime = FApp::GetCurrentTime() - It->LastReceivedMessageTime;
		if (DeltaTime > GetDefault<UStageMonitoringSettings>()->TimeoutInterval)
		{
			UE_LOGF(LogStageDataProvider, Log, "Monitor lost, %ls|%d", *It->Descriptor.MachineName, It->Descriptor.ProcessId);
			It.RemoveCurrent();
		}
	}
}

bool FStageDataProvider::IsMessageTypeExcluded(UScriptStruct* MessageType) const
{
	check(MessageType);
	const FStageDataProviderSettings& ProviderSettings = GetDefault<UStageMonitoringSettings>()->ProviderSettings;
	if (const FGameplayTagContainer* SupportedRoles = ProviderSettings.MessageTypeRoleExclusion.Find(FStageMessageTypeWrapper(MessageType->GetFName())))
	{
		if (GEngine)
		{
			if (UVirtualProductionRolesSubsystem* VPRolesSubsytem = GEngine->GetEngineSubsystem<UVirtualProductionRolesSubsystem>())
			{
				const FGameplayTagContainer& CurrentRoles = VPRolesSubsytem->GetRolesContainer_Private();
				if (!SupportedRoles->HasAny(CurrentRoles))
				{
					return true;
				}
			}
		}
	}

	return false;
}

