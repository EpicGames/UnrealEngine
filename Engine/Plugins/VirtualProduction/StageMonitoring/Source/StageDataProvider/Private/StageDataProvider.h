// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStageDataProvider.h"

#include "IMessageContext.h"
#include "Containers/Ticker.h"

class FMessageEndpoint;
struct FStageDataBaseMessage; 

/**
 * Implementation of the stage data provider.
 * Uses message bus to send messages to monitors
 */
class FStageDataProvider : public IStageDataProvider
{
public:
	FStageDataProvider();
	virtual ~FStageDataProvider();

	/** Setup everything required to be able to listen for monitors and send data */
	void Start();

	/** Let monitors know we're closing and cleanup endpoint */
	void Stop();

protected:

	//~Begin IStageDataProvider interface
	virtual bool SendMessageInternal(FStageDataBaseMessage* Payload, UScriptStruct* Type, EStageMessageFlags InFlags) override;
	//~End IStageDataProviderinterface

private:

	/** Handles reception of a discovery message sent by monitors. Will send descriptor back if its coming from a new monitor */
	void HandleDiscoveryMessage(const FStageProviderDiscoveryMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	
	/** Handles reception of a monitor closing message. Removes it from the linked monitor list */
	void HandleMonitorCloseMessage(const FStageMonitorCloseMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	
	/** Used to verify if monitors have timedout */
	bool Tick(float DeltaTime);

	/** Remove monitors for which we haven't received any message for a while */
	void RemoveTimeoutedMonitors();

	/** Verify if this message type is excluded using project settings */
	bool IsMessageTypeExcluded(UScriptStruct* MessageType) const;


private:

	/** Holds information about monitors that discovered us */
	struct FMonitorEndpoints
	{
		FGuid Identifier;
		FStageInstanceDescriptor Descriptor;
		FMessageAddress Address;
		double LastReceivedMessageTime = 0.0;
	};

	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> ProviderEndpoint;
	TArray<FMonitorEndpoints> Monitors;

	/** Used to easily convert to an array when sending to the attached monitors */
	TArray<FMessageAddress> CachedMonitorAddresses;
	bool bIsActive = false;
	FTSTicker::FDelegateHandle TickerHandle;

	/** Used to keep track of monitors that were not compatible with us to avoid log spamming */
	TArray<FGuid> InvalidMonitors;

	/** Identifier of this provider */
	FGuid Identifier;
};
