// Copyright Epic Games, Inc. All Rights Reserved.

#include "StageDataProviderModule.h"

#include "Engine/Engine.h"
#include "FramePerformanceProvider.h"
#include "GenlockWatchdog.h"
#include "Misc/CoreDelegates.h"
#include "StageDataProvider.h"
#include "TakeRecorderStateProvider.h"
#include "TimecodeProviderWatchdog.h"
#include "VPRolesSubsystem.h"


const FName IStageDataProviderModule::ModuleName = TEXT("StageDataProvider");


DEFINE_LOG_CATEGORY(LogStageDataProvider);

FStageDataProviderModule::FStageDataProviderModule()
	: DataProvider(MakeUnique<FStageDataProvider>())
	, FramePerformanceProvider(MakeUnique<FFramePerformanceProvider>())
	, GenlockWatchdog(MakeUnique<FGenlockWatchdog>())
	, TimecodeWatchdog(MakeUnique<FTimecodeProviderWatchdog>())
#if WITH_EDITOR
	, TakeRecorderStateProvider(MakeUnique<FTakeRecorderStateProvider>())
#endif //WITH_EDITOR
{
}

void FStageDataProviderModule::StartupModule()
{
	//Postpone initialization to make sure modules are loaded. We depend on other settings that could still be unloaded
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FStageDataProviderModule::OnPostEngineInit);

	//Delegate used to stop the provider before being completely closed down
	FCoreDelegates::OnPreExit.AddRaw(this, &FStageDataProviderModule::OnPreExit);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	CommandStart = MakeUnique<FAutoConsoleCommand>(TEXT("StageMonitor.DataProvider.Start")
												, TEXT("Start Stage data provider")
												, FConsoleCommandDelegate::CreateRaw(this, &FStageDataProviderModule::StartDataProvider));

	CommandStop = MakeUnique<FAutoConsoleCommand>(TEXT("StageMonitor.DataProvider.Stop")
												, TEXT("Stop Stage data provider")
												, FConsoleCommandDelegate::CreateRaw(this, &FStageDataProviderModule::StopDataProvider));
#endif
}

void FStageDataProviderModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	FCoreDelegates::OnPreExit.RemoveAll(this);
	StopDataProvider();
}

void FStageDataProviderModule::OnPostEngineInit()
{
	const UVirtualProductionRolesSubsystem* RolesSubsystem = GEngine->GetEngineSubsystem<UVirtualProductionRolesSubsystem>();
	//If role enforcement is enabled, make sure we have one required to enable data provider
	const UStageMonitoringSettings* Settings = GetDefault<UStageMonitoringSettings>();
	if (!Settings->ProviderSettings.bUseRoleFiltering || RolesSubsystem->GetRolesContainer_Private().HasAny(Settings->ProviderSettings.SupportedRoles))
	{
		StartDataProvider();
	}
	else
	{
		UE_LOGF(LogStageDataProvider, Log, "Can't start StageDataProvider. Role filtering is enabled and our roles (%ls) are not accepted (%ls)"
		, *RolesSubsystem->GetActiveRolesString()
		, *Settings->ProviderSettings.SupportedRoles.ToStringSimple());
	}
}

void FStageDataProviderModule::OnPreExit()
{
	StopDataProvider();
}

void FStageDataProviderModule::StartDataProvider()
{
	DataProvider->Start();

	//Register the provider to the system to make it available engine wide
	IModularFeatures::Get().RegisterModularFeature(IStageDataProvider::ModularFeatureName, DataProvider.Get());
}

void FStageDataProviderModule::StopDataProvider()
{
	IModularFeatures::Get().UnregisterModularFeature(IStageDataProvider::ModularFeatureName, DataProvider.Get());
	DataProvider->Stop();
}

IMPLEMENT_MODULE(FStageDataProviderModule, StageDataProvider);
