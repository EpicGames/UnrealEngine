// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStageDataProviderModule.h"


class FAutoConsoleCommand;
class FFramePerformanceProvider;
class FGenlockWatchdog;
class FStageDataProvider;
class FTakeRecorderStateProvider;
class FTimecodeProviderWatchdog;

DECLARE_LOG_CATEGORY_EXTERN(LogStageDataProvider, Log, All);

/**
 * Interface for the stage data provider module.
 */
class FStageDataProviderModule : public IStageDataProviderModule
{
public:
	FStageDataProviderModule();

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

private:

	/** We rely on VPUtilities module, make sure everything is loaded before we use it */
	void OnPostEngineInit();

	/** WHen exiting the app, stop data providers so data providers can be closed */
	void OnPreExit();

	/** Starts data provider */
	void StartDataProvider();

	/** Stops data provider */
	void StopDataProvider();

private:

	/** Instance of the data provider */
	TUniquePtr<FStageDataProvider> DataProvider;
	
	/** Frame performance provider instance sending information about hitch and performance */
	TUniquePtr<FFramePerformanceProvider> FramePerformanceProvider;

	/** Monitors the genlock custom timestep, if available, to notify of hitches */
	TUniquePtr<FGenlockWatchdog> GenlockWatchdog;

	/** Monitors the TimecodeProvider, if available, to notify of state change or missed frames */
	TUniquePtr<FTimecodeProviderWatchdog> TimecodeWatchdog;

#if WITH_EDITOR
	/** TakeRecorder provider instance sends information about stage critical state */
	TUniquePtr<FTakeRecorderStateProvider> TakeRecorderStateProvider;
#endif //WITH_EDITOR

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	TUniquePtr<FAutoConsoleCommand> CommandStart;
	TUniquePtr<FAutoConsoleCommand> CommandStop;
#endif
};
