// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "TakeRecorderStateProvider.h"

#include "IStageDataProvider.h"
#include "LevelSequence.h"
#include "Recorder/TakeRecorder.h"


FTakeRecorderStateProvider::FTakeRecorderStateProvider()
{
	UTakeRecorder::OnRecordingInitialized().AddRaw(this, &FTakeRecorderStateProvider::OnTakeRecorderInitializer);
}

void FTakeRecorderStateProvider::OnTakeRecorderInitializer(UTakeRecorder* InRecorder)
{
	if (InRecorder)
	{
		//When recording is started, we'll send a reliable message with the current timecode so monitors can know the start time boundary for the critical state
		InRecorder->OnRecordingStarted().AddRaw(this, &FTakeRecorderStateProvider::OnTakeRecorderStarted);

		//When recording is finished or cancelled, we'll send a reliable message with the current timecode so monitors can know the end time boundary for the critical state
		InRecorder->OnRecordingCancelled().AddRaw(this, &FTakeRecorderStateProvider::OnTakeRecorderStopped);
		InRecorder->OnRecordingFinished().AddRaw(this, &FTakeRecorderStateProvider::OnTakeRecorderStopped);
	}

	CurrentState = EStageCriticalStateEvent::Exit;
}

void FTakeRecorderStateProvider::OnTakeRecorderStarted(UTakeRecorder* InRecorder)
{
	if (CurrentState == EStageCriticalStateEvent::Exit)
	{
		TakeName = InRecorder->GetSequence()->GetFName();
		CurrentState = EStageCriticalStateEvent::Enter;
		IStageDataProvider::SendMessage<FCriticalStateProviderMessage>(EStageMessageFlags::Reliable, CurrentState, TakeName);
	}
}

void FTakeRecorderStateProvider::OnTakeRecorderStopped(UTakeRecorder* InRecorder)
{
	if (CurrentState == EStageCriticalStateEvent::Enter)
	{
		CurrentState = EStageCriticalStateEvent::Exit;
		IStageDataProvider::SendMessage<FCriticalStateProviderMessage>(EStageMessageFlags::Reliable, CurrentState, TakeName);
	}
}

#endif //WITH_EDITOR


