// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/** Filtering out this class entirely since its useless without it */
#if WITH_EDITOR


#include "StageMessages.h"


class UTakeRecorder;

class FTakeRecorderStateProvider
{
public:
	FTakeRecorderStateProvider();

private:
	/** When a take recorder is initialized, hooks us to different delegates to know its state */
	void OnTakeRecorderInitializer(UTakeRecorder* InRecorder);

	/** Sends out active state based on take recorder */
	void OnTakeRecorderStarted(UTakeRecorder* InRecorder);

	/** Sends out inactive state based on take recorder */
	void OnTakeRecorderStopped(UTakeRecorder* InRecorder);

private:

	/** Name of the take being done */
	FName TakeName;

	/** Current state of this stage critical status */
	EStageCriticalStateEvent CurrentState = EStageCriticalStateEvent::Exit;
};


#endif //WITH_EDITOR
