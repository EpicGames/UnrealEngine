// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimecodeProviderWatchdog.h"

#include "Engine/Engine.h"
#include "IStageDataProvider.h"
#include "Misc/CoreDelegates.h"


FTimecodeProviderWatchdog::FTimecodeProviderWatchdog()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FTimecodeProviderWatchdog::OnPostEngineInit);
	FCoreDelegates::OnEndFrame.AddRaw(this, &FTimecodeProviderWatchdog::OnEndFrame);
}

FTimecodeProviderWatchdog::~FTimecodeProviderWatchdog()
{
	FCoreDelegates::OnEndFrame.RemoveAll(this);

	if (GEngine)
	{
		GEngine->OnTimecodeProviderChanged().RemoveAll(this);
	}

	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
}

void FTimecodeProviderWatchdog::OnEndFrame()
{
	UpdateProviderState();
}

void FTimecodeProviderWatchdog::OnPostEngineInit()
{
	if (GEngine)
	{
		GEngine->OnTimecodeProviderChanged().AddRaw(this, &FTimecodeProviderWatchdog::OnTimecodeProviderChanged);

		CacheProviderInfo();
		UpdateProviderState();
	}
}

void FTimecodeProviderWatchdog::OnTimecodeProviderChanged()
{
	//If a state was set, it means we had a provider cached
	if (LastState.IsSet())
	{
		IStageDataProvider::SendMessage<FTimecodeProviderStateEvent>(EStageMessageFlags::Reliable, ProviderName, ProviderType, FrameRate, ETimecodeProviderSynchronizationState::Closed);
	}

	CacheProviderInfo();
	UpdateProviderState();
}

void FTimecodeProviderWatchdog::CacheProviderInfo()
{
	UTimecodeProvider* Provider = GEngine->GetTimecodeProvider();
	if (Provider)
	{
		ProviderName = Provider->GetName();
		ProviderType = Provider->GetClass()->GetName();
	}

	LastState.Reset();
}

void FTimecodeProviderWatchdog::UpdateProviderState()
{
	if (GEngine)
	{
		UTimecodeProvider* Provider = GEngine->GetTimecodeProvider();
		if (Provider)
		{
			FrameRate = Provider->GetFrameRate();
			const ETimecodeProviderSynchronizationState State = Provider->GetSynchronizationState();
			if (!LastState.IsSet() || State != LastState.GetValue())
			{
				IStageDataProvider::SendMessage<FTimecodeProviderStateEvent>(EStageMessageFlags::Reliable, ProviderName, ProviderType, FrameRate, State);
				LastState = State;
			}
		}
	}
}

FString FTimecodeProviderStateEvent::ToString() const
{
	return FString::Printf(TEXT("%s (%s) state is now: %s"), *ProviderName, *ProviderType, *StaticEnum<ETimecodeProviderSynchronizationState>()->GetNameStringByValue((int64)NewState));
}
