// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StageMessages.h"

#include "Engine/TimecodeProvider.h"

#include "TimecodeProviderWatchdog.generated.h"


/**
 * Stage event to notify of TimecodeProvider state change
 */
USTRUCT()
struct FTimecodeProviderStateEvent : public FStageProviderEventMessage
{
	GENERATED_BODY()

public:
	FTimecodeProviderStateEvent() = default;
	FTimecodeProviderStateEvent(FString InProviderName, FString InProviderType, FFrameRate InFrameRate, ETimecodeProviderSynchronizationState InState)
		: ProviderName(MoveTemp(InProviderName))
		, ProviderType(MoveTemp(InProviderType))
		, FrameRate(MoveTemp(InFrameRate))
		, NewState(InState)
		{}

	virtual FString ToString() const override;

	/** Name of the TimeodeProvider for this event */
	UPROPERTY(VisibleAnywhere, Category = "Timecode")
	FString ProviderName;

	/** Type of the TimecodeProvider for this event */
	UPROPERTY(VisibleAnywhere, Category = "Timecode")
	FString ProviderType;
	
	/** FrameRate of the provider */
	UPROPERTY(VisibleAnywhere, Category = "Timecode")
	FFrameRate FrameRate;

	/** New state of TimecodeProvider (i.e. Synchronized, Error, etc...) */
	UPROPERTY(VisibleAnywhere, Category = "Timecode")
	ETimecodeProviderSynchronizationState NewState = ETimecodeProviderSynchronizationState::Closed;
};

/**
 * Simple TimecodeProvider watcher notifying stage mon if state changes
 * Timecode validation could also be added in the future
 */
class FTimecodeProviderWatchdog 
{
public:
	FTimecodeProviderWatchdog();
	~FTimecodeProviderWatchdog();

private:

	/** At end of frame, verify timecode state and notify stage monitor if something changed */
	void OnEndFrame();

	/** Delegate post engine to have access to GEngine and get current TimecodeProvider */
	void OnPostEngineInit();

	/** Detect when a provider has changed to send a closed notification for the previous one */
	void OnTimecodeProviderChanged();

	/** Cache info about the current provider if there is one */
	void CacheProviderInfo();

	/** Update internal TimecodeProvider state and trigger event if it has changed or first time checked */
	void UpdateProviderState();

private:
	
	/** Previous value of the timecode provider state. */
	TOptional<ETimecodeProviderSynchronizationState> LastState;

	/**  Cached provider name */
	FString ProviderName;

	/** Cached provider class type */
	FString ProviderType;

	/** Last FrameRate provider had */
	FFrameRate FrameRate;
};
