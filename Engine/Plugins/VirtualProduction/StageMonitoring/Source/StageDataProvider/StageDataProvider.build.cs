// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class StageDataProvider : ModuleRules
{
	public StageDataProvider(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"RHI",
				"GameplayTags",
				"RenderCore",
				"StageDataCore",
				"StageMonitorCommon",
				"TimeManagement",
				"VPRoles",
				"VPUtilities",
			}
		);

		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"LevelSequence",
					"TakeRecorder",
				}
			);
		}
	}
}
