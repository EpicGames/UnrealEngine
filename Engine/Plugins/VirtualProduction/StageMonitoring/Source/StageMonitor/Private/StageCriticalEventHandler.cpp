// Copyright Epic Games, Inc. All Rights Reserved.

#include "StageCriticalEventHandler.h"
#include "StageMessages.h"

bool FStageCriticalEventHandler::IsTimingPartOfCriticalRange(double TimeInSeconds) const
{
	if (IsCriticalStateActive())
	{
		if (CurrentState->Range.Contains(TimeInSeconds))
		{
			return true;
		}
	}

	for (const FCriticalStateEntry& Entry : CriticalStateHistory)
	{
		if (Entry.Range.Contains(TimeInSeconds))
		{
			return true;
		}
	}

	return false;
}

void FStageCriticalEventHandler::HandleCriticalEventMessage(const FCriticalStateProviderMessage* Message)
{
	const FGuid& Identifier = Message->Identifier;
	if (Message->State == EStageCriticalStateEvent::Enter)
	{
		if (!IsCriticalStateActive())
		{
			FCriticalStateEntry NewEntry;
			NewEntry.Range = TRange<double>::AtLeast(Message->FrameTime.AsSeconds());
			CurrentState.Emplace(MoveTemp(NewEntry));
		}

		//Keep track of all sources for this state entry
		CurrentState->Sources.AddUnique(Message->SourceName);

		// Add the event to our active list. Receiving multiple 'enter' for the same identifier and source will have no effect
		ActiveEvents.FindOrAdd(Identifier).AddUnique(Message->SourceName);
	}
	else
	{
		if (ActiveEvents.Contains(Identifier))
		{
			ActiveEvents[Identifier].RemoveSingleSwap(Message->SourceName, EAllowShrinking::No);
			
			//Verify if this machine has no more events active
			if (ActiveEvents[Identifier].Num() <= 0)
			{
				ActiveEvents.Remove(Identifier);
			}
		}

		if (IsCriticalStateActive())
		{
			//Verify if we don't have active events anymore
			if (ActiveEvents.Num() <= 0)
			{
				CloseCriticalState(Message->FrameTime.AsSeconds());
			}
		}
	}
}

void FStageCriticalEventHandler::RemoveProviderEntries(double CurrentTime, const FGuid& Identifier)
{
	ActiveEvents.Remove(Identifier);

	if (IsCriticalStateActive())
	{
		//Verify if we don't have active events anymore
		if (ActiveEvents.Num() <= 0)
		{
			CloseCriticalState(CurrentTime);
		}
	}
}

FName FStageCriticalEventHandler::GetCurrentCriticalStateSource() const
{
	if (IsCriticalStateActive())
	{
		//Returns first Source
		const TArray<FName>& Sources = ActiveEvents.CreateConstIterator().Value();
		return Sources[0];
	}

	return NAME_None;
}

TArray<FName> FStageCriticalEventHandler::GetCriticalStateHistorySources() const
{
	TArray<FName> Sources;
	for (const FCriticalStateEntry& Entry : CriticalStateHistory)
	{
		for (const FName& Source : Entry.Sources)
		{
			Sources.AddUnique(Source);
		}
	}

	if (IsCriticalStateActive())
	{
		for (const FName& Source : CurrentState->Sources)
		{
			Sources.AddUnique(Source);
		}
	}

	return Sources;
}

TArray<FName> FStageCriticalEventHandler::GetCriticalStateSources(double TimeInSeconds) const
{
	TArray<FName> Sources;

	if (IsCriticalStateActive())
	{
		if (CurrentState->Range.Contains(TimeInSeconds))
		{
			for (const FName& Entry : CurrentState->Sources)
			{
				Sources.AddUnique(Entry);
			}
		}
	}

	for (const FCriticalStateEntry& Entry : CriticalStateHistory)
	{
		if (Entry.Range.Contains(TimeInSeconds))
		{
			for (const FName& Source : Entry.Sources)
			{
				Sources.AddUnique(Source);
			}
		}
	}

	return Sources;
}

void FStageCriticalEventHandler::CloseCriticalState(double CurrentSeconds)
{
	if (IsCriticalStateActive())
	{
		//Verify if we don't have active events anymore
		if (ActiveEvents.Num() <= 0)
		{
			//Exit critical state
			CurrentState->Range.SetUpperBound(CurrentSeconds);

			//Add that to our history
			CriticalStateHistory.Emplace(MoveTemp(CurrentState.GetValue()));

			//Reset current range since it's not active anymore
			CurrentState.Reset();
		}
	}
}
