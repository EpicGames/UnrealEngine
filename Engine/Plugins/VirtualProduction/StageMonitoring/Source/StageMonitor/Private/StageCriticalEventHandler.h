// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Containers/Map.h"
#include "Math/Range.h"
#include "Misc/Guid.h"

struct FCriticalStateProviderMessage;


/**
 * Holds information about what caused a critical state and for what timerange
 * The range starts at first critical state entry and ends when last critical states exits
 */
struct FCriticalStateEntry
{
	TArray<FName> Sources;
	TRange<double> Range;
};


/**
 * Stage monitoring becomes critical when it's in a state where accuracy, timing, performance is essential
 * This class manages the state of the stage. Multiple sources can set the stage in critical state.
 * All of them needs to close for the stage state to come back as normal.
 */
class FStageCriticalEventHandler
{
public:

	/** Whether or not a critical state is currently active */
	bool IsCriticalStateActive() const { return CurrentState.IsSet(); }

	/** Verify if a time is part of any critical state range we've encountered, including the active one */
	bool IsTimingPartOfCriticalRange(double TimeInSeconds) const;

	/** Process new critical event message and update current state accordingly */
	void HandleCriticalEventMessage(const FCriticalStateProviderMessage* Message);

	/** Used to clean up the event handler in case a provider is closed, while critical events associated to it are opened */
	void RemoveProviderEntries(double CurrentTime, const FGuid& Identifier);

	/** Returns the source name of the current critical state */
	FName GetCurrentCriticalStateSource() const;

	/** Returns the sources that triggered a critical state at some point */
	TArray<FName> GetCriticalStateHistorySources() const;

	/** Returns the sources of the critical state that provided time is in. Empty is not critical state found */
	TArray<FName> GetCriticalStateSources(double TimeInSeconds) const;

private:
	/** Close critical state at specified timestamp if only one layer is active */
	void CloseCriticalState(double CurrentSeconds);

private:
	/** 
	 * Current active events per provider. A provider can trigger more than one type of events
	 * When no more providers have active events, the critical event range is closed
	 */
	TMap<FGuid, TArray<FName>> ActiveEvents;

	/** Timing range of the current critical state. Unbounded until it's completed */
	TOptional<FCriticalStateEntry> CurrentState;

	/** History of timing ranges when stage was in critical state */
	TArray<FCriticalStateEntry> CriticalStateHistory;
};
