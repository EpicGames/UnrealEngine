// Copyright Epic Games, Inc. All Rights Reserved.

#include "StageMonitor.h"

#include "IStageMonitorSession.h"
#include "IStageMonitorSessionManager.h"
#include "MessageEndpointBuilder.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "StageMonitoringSettings.h"
#include "StageMonitorModule.h"
#include "StageMonitorUtils.h"


namespace StageMonitorUtils
{
	//Stagemessage version we support
	static const int32 SupportedStageMessageVersion = 1;
}

FStageMonitor::~FStageMonitor()
{
	Stop();
	FCoreDelegates::OnPreExit.RemoveAll(this);
}

void FStageMonitor::Initialize()
{
	FCoreDelegates::OnPreExit.AddRaw(this, &FStageMonitor::OnPreExit);

	Session = IStageMonitorModule::Get().GetStageMonitorSessionManager().CreateSession();

	bIsActive = false;
	Identifier = FGuid::NewGuid();

	CachedDiscoveryMessage.Descriptor = StageMonitorUtils::GetInstanceDescriptor();
}

void FStageMonitor::Start()
{
	if (bIsActive == false)
	{
		const FString MessageEndpointName = TEXT("StageDataMonitor");
		MonitorEndpoint = FMessageEndpoint::Builder(*MessageEndpointName)
			.ReceivingOnThread(ENamedThreads::Type::GameThread)
			.Handling<FStageProviderDiscoveryResponseMessage>(this, &FStageMonitor::HandleProviderDiscoveryResponse)
			.Handling<FStageProviderCloseMessage>(this, &FStageMonitor::HandleProviderCloseMessage)
			.WithCatchall(this, &FStageMonitor::HandleStageData);

		if (MonitorEndpoint.IsValid())
		{
			MonitorEndpoint->Subscribe<FStageProviderDiscoveryResponseMessage>();
		}

		TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FStageMonitor::Tick), 0.0f);

		bIsActive = true;
	}
}

void FStageMonitor::Stop()
{
	bIsActive = false;

	if (TickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickerHandle);
		TickerHandle.Reset();
	}

	//Let providers know we're out
	if (MonitorEndpoint.IsValid())
	{
		SendMessage<FStageMonitorCloseMessage>(EStageMessageFlags::Reliable);

		// Disable the Endpoint message handling since the message could keep it alive a bit.
		MonitorEndpoint->Disable();
		MonitorEndpoint.Reset();
	}

	InvalidDataProviders.Empty();
}

bool FStageMonitor::Tick(float DeltaTime)
{
	UpdateProviderState();
	SendDiscoveryMessage();

	return true;
}

void FStageMonitor::UpdateProviderState()
{
	const TConstArrayView<FStageSessionProviderEntry> Providers = Session->GetProviders();
	for (const FStageSessionProviderEntry& Provider : Providers)
	{
		if (Provider.State == EStageDataProviderState::Active)
		{
			if (FApp::GetCurrentTime() - Provider.LastReceivedMessageTime > GetDefault<UStageMonitoringSettings>()->TimeoutInterval)
			{
				UE_LOGF(LogStageMonitor, VeryVerbose, "Provider lost, %ls", *Provider.Descriptor.FriendlyName.ToString());
				Session->SetProviderState(Provider.Identifier, EStageDataProviderState::Inactive);
			}
		}
		else if (Provider.State == EStageDataProviderState::Inactive)
		{
			if (FApp::GetCurrentTime() - Provider.LastReceivedMessageTime <= GetDefault<UStageMonitoringSettings>()->TimeoutInterval)
			{
				UE_LOGF(LogStageMonitor, VeryVerbose, "Provider back alive, %ls", *Provider.Descriptor.FriendlyName.ToString());
				Session->SetProviderState(Provider.Identifier, EStageDataProviderState::Active);
			}
		}
		else if (Provider.State == EStageDataProviderState::Closed)
		{
				UE_LOGF(LogStageMonitor, Log, "Provider is closed, %ls", *Provider.Descriptor.FriendlyName.ToString());
		}
	}
}

void FStageMonitor::SendDiscoveryMessage()
{
	if (FApp::GetCurrentTime() - LastSentDiscoveryMessage > GetDefault<UStageMonitoringSettings>()->MonitorSettings.DiscoveryMessageInterval)
	{
		//Broadcast discovery signal
		PublishMessage<FStageProviderDiscoveryMessage>(CachedDiscoveryMessage);

		LastSentDiscoveryMessage = FApp::GetCurrentTime();
	}
}

void FStageMonitor::OnPreExit()
{
	//If we are exiting, close ourself so we can notify listeners
	Stop();
}

bool FStageMonitor::SendMessageInternal(FStageDataBaseMessage* Payload, UScriptStruct* Type, EStageMessageFlags InFlags)
{
	TArray<FMessageAddress> Addresses = Session->GetProvidersAddress();
	if (Addresses.Num() > 0)
	{
		//Message bus requires payload to be newed and life cycle will be taken care of in MB
		FStageDataBaseMessage* MessageData = reinterpret_cast<FStageDataBaseMessage*>(FMemory::Malloc(Type->GetStructureSize()));
		Type->InitializeStruct(MessageData);
		Type->CopyScriptStruct(MessageData, Payload);

		//Set identifier in sent message
		MessageData->Identifier = Identifier;

		const EMessageFlags Flags = EnumHasAnyFlags(InFlags, EStageMessageFlags::Reliable) ? EMessageFlags::Reliable : EMessageFlags::None;
		MonitorEndpoint->Send(MessageData, Type, Flags, nullptr, Addresses, FTimespan::Zero(), FDateTime::MaxValue());
		return true;
	}

	return false;
}

void FStageMonitor::PublishMessageInternal(FStageDataBaseMessage* Payload, UScriptStruct* Type)
{
	//Message bus requires payload to be newed and life cycle will be taken care of in MB
	FStageDataBaseMessage* MessageData = reinterpret_cast<FStageDataBaseMessage*>(FMemory::Malloc(Type->GetStructureSize()));
	Type->InitializeStruct(MessageData);
	Type->CopyScriptStruct(MessageData, Payload);

	//Set identifier in sent message
	MessageData->Identifier = Identifier;

	MonitorEndpoint->Publish(MessageData, Type, EMessageScope::Network, FTimespan::Zero(), FDateTime::MaxValue());
}

void FStageMonitor::HandleProviderDiscoveryResponse(const FStageProviderDiscoveryResponseMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	//For now, work explicitely for the current version. If things change later on, we can adapt how we handle this
	if (Message.StageMessageVersion != StageMonitorUtils::SupportedStageMessageVersion)
	{
		//Keep track of invalid providers to avoid log spamming
		if (!InvalidDataProviders.Contains(Message.Identifier))
		{
			InvalidDataProviders.Add(Message.Identifier);
			
			UE_LOGF(LogStageMonitor, Warning, "Received discovery response from Provider (%ls:%d) with unsupported version '%d'. Supported version is '%d'"
				, *Message.Descriptor.MachineName
				, Message.Descriptor.ProcessId
				, Message.StageMessageVersion
				, StageMonitorUtils::SupportedStageMessageVersion);
		}

		return;
	}

	//SessionId can be used to filter messages coming from another stage, for example. 
	//If session id filtering is enabled, make sure messages are coming from our stage
	const UStageMonitoringSettings* Settings = GetDefault<UStageMonitoringSettings>();
	if (!Settings->bUseSessionId || Settings->GetStageSessionId() == Message.Descriptor.SessionId)
	{
		Session->HandleDiscoveredProvider(Message.Identifier, Message.Descriptor, Context->GetSender());
	}
	else if (!InvalidDataProviders.Contains(Message.Identifier))
	{
		//Keep track of invalid providers to avoid log spamming
		InvalidDataProviders.Add(Message.Identifier);

		UE_LOGF(LogStageMonitor, Warning, "Received discovery response from Provider (%ls:%d) with SessionId '%d' but expected '%d'"
		, *Message.Descriptor.MachineName
		, Message.Descriptor.ProcessId
		, Message.Descriptor.SessionId
		, Settings->GetStageSessionId());
	}
}

void FStageMonitor::HandleProviderCloseMessage(const FStageProviderCloseMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	const FGuid ProviderIdentifier = Message.Identifier;
	const TConstArrayView<FStageSessionProviderEntry> Providers = Session->GetProviders();
	const FStageSessionProviderEntry* Provider = Providers.FindByPredicate([ProviderIdentifier](const FStageSessionProviderEntry& Other) { return Other.Identifier == ProviderIdentifier; });
	if (Provider)
	{
		Session->SetProviderState(ProviderIdentifier, EStageDataProviderState::Closed);
	}
}

void FStageMonitor::HandleStageData(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	//Verify message validity
	if (!Context->IsValid())
	{
		return;
	}

	UScriptStruct* MessageTypeInfo = Context->GetMessageTypeInfo().Get();
	if (MessageTypeInfo == nullptr)
	{
		return;
	}

	const bool bIsProviderMessage = MessageTypeInfo->IsChildOf(FStageProviderMessage::StaticStruct());
	if (!bIsProviderMessage)
	{
		return;
	}

	const FStageProviderMessage* Message = reinterpret_cast<const FStageProviderMessage*>(Context->GetMessage());
	check(Message);

	// Pool that message in the session
	Session->AddProviderMessage(MessageTypeInfo, Message);
}

