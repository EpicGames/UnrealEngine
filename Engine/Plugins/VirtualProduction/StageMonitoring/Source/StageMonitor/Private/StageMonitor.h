// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStageMonitor.h"

#include "Containers/Ticker.h"

#include "StageMessages.h"

class IStageMonitorSession;

class FMessageEndpoint;
struct FStageDataBaseMessage;
class IMessageContext;
class FStageMonitorSession;

/**
 * Implementation of the stage monitor
 */
class FStageMonitor : public IStageMonitor
{
public:

	FStageMonitor() = default;
	FStageMonitor(const FStageMonitor&) = delete;
	FStageMonitor& operator=(const FStageMonitor&) = delete;
	FStageMonitor(FStageMonitor&&) = default;
	FStageMonitor& operator=(FStageMonitor&&) = default;
	virtual ~FStageMonitor();

	//~Begin IStageMonitor interface
	virtual bool IsActive() const override { return bIsActive; }
	//~End IStageMonitor


	void Initialize();
	void Start();
	void Stop();

private:

	/** Sends a message using messagebus to connected providers */
	bool SendMessageInternal(FStageDataBaseMessage* Payload, UScriptStruct* Type, EStageMessageFlags InFlags);

	/** Broadcast a message on messagebus */
	void PublishMessageInternal(FStageDataBaseMessage* Payload, UScriptStruct* Type);

	/** Catchall messages of messagebus to handle DataProvider messages */
	void HandleStageData(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handles discovery response coming from a data provider */
	void HandleProviderDiscoveryResponse(const FStageProviderDiscoveryResponseMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handles Close message from a provider shutting down */
	void HandleProviderCloseMessage(const FStageProviderCloseMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/**  */
	bool Tick(float DeltaTime);

	/** Updates each provider's state based on last communication timestamp */
	void UpdateProviderState();

	/** Broadcasts a discovery message to find new providers and also keep communication active with all providers */
	void SendDiscoveryMessage();

	/** Shutdowns monitor before everything has exited. Let know providers that we're out */
	void OnPreExit();

	/** Generic send message to support constructor parameters directly and temp object created */
	template<typename MessageType, typename... Args>
	bool SendMessage(EStageMessageFlags Flags, Args&&... args)
	{
#if ENABLE_STAGEMONITOR_LOGGING
		static_assert(TIsDerivedFrom<MessageType, FStageMonitorBaseMessage>::IsDerived, "MessageType must be a FStageMonitorBaseMessage derived UStruct.");
		MessageType TempObj(Forward<Args>(args)...);
		return SendMessageInternal(&TempObj, MessageType::StaticStruct(), Flags);
#endif
		return false;
	}

	/** Generic publish message to support constructor parameters directly and temp object created */
	template<typename MessageType, typename... Args>
	void PublishMessage(Args&&... args)
	{
#if ENABLE_STAGEMONITOR_LOGGING
		static_assert(TIsDerivedFrom<MessageType, FStageMonitorBaseMessage>::IsDerived, "MessageType must be a FStageMonitorBaseMessage derived UStruct.");
		MessageType TempObj(Forward<Args>(args)...);
		PublishMessageInternal(&TempObj, MessageType::StaticStruct());
#endif
	}

private:

	/** Endpoint used to communicate with providers */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MonitorEndpoint;

	/** List of providers we received discovery response from that we not usable (wrong version, session id...) */
	TArray<FGuid> InvalidDataProviders;

	/** Current active session where we dispatch new data */
	TSharedPtr<IStageMonitorSession> Session;

	/** Timestamp when we last sent a discovery message */
	double LastSentDiscoveryMessage = 0.0;

	/** Whether monitor is active (sending out discovery, listening to providers) or not */
	bool bIsActive = false;

	/** Handle to our ticking delegate (Ticks only when bIsActive is true) */
	FTSTicker::FDelegateHandle TickerHandle;

	/** This monitor identifier */
	FGuid Identifier;

	/** DiscoveryMessage built once and broadcasted periodically */
	FStageProviderDiscoveryMessage CachedDiscoveryMessage;
};
