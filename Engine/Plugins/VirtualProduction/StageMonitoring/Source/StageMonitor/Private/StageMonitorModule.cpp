// Copyright Epic Games, Inc. All Rights Reserved.

#include "StageMonitorModule.h"

#include "Engine/Engine.h"
#include "Misc/CoreDelegates.h"
#include "StageMonitor.h"
#include "StageMonitorSessionManager.h"
#include "VPRolesSubsystem.h"

const FName IStageMonitorModule::ModuleName = TEXT("StageMonitor");


DEFINE_LOG_CATEGORY(LogStageMonitor)


void FStageMonitorModule::StartupModule()
{
	FCoreDelegates::OnFEngineLoopInitComplete.AddRaw(this, &FStageMonitorModule::OnEngineLoopInitComplete);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	CommandStart = MakeUnique<FAutoConsoleCommand>(TEXT("StageMonitor.Monitor.Start")
													, TEXT("Start Stage monitoring")
													, FConsoleCommandDelegate::CreateRaw(this, &FStageMonitorModule::EnableMonitor, true));

	CommandStop = MakeUnique<FAutoConsoleCommand>(TEXT("StageMonitor.Monitor.Stop")
												, TEXT("Stop Stage monitoring")
												, FConsoleCommandDelegate::CreateRaw(this, &FStageMonitorModule::EnableMonitor, false));
#endif 
}

void FStageMonitorModule::ShutdownModule()
{
	FCoreDelegates::OnFEngineLoopInitComplete.RemoveAll(this);
	StageMonitor.Reset();
}

void FStageMonitorModule::OnEngineLoopInitComplete()
{
	SessionManager = MakeUnique<FStageMonitorSessionManager>();
	StageMonitor = MakeUnique<FStageMonitor>();
	StageMonitor->Initialize();

	const UVirtualProductionRolesSubsystem* RolesSubsystem = GEngine->GetEngineSubsystem<UVirtualProductionRolesSubsystem>();

	const UStageMonitoringSettings* Settings = GetDefault<UStageMonitoringSettings>();
	if (Settings->MonitorSettings.ShouldAutoStartOnLaunch())
	{
		if (!Settings->MonitorSettings.bUseRoleFiltering || RolesSubsystem->GetRolesContainer_Private().HasAny(Settings->MonitorSettings.SupportedRoles))
		{
			EnableMonitor(true);
		}
		else
		{
			UE_LOGF(LogStageMonitor, Log, "Can't autostart StageMonitor. Role filtering is enabled and our roles (%ls) are filtered out (%ls)"
				, *RolesSubsystem->GetActiveRolesString()
				, *Settings->MonitorSettings.SupportedRoles.ToStringSimple())
		}
	}
}

IStageMonitor& FStageMonitorModule::GetStageMonitor()
{
	return *StageMonitor;
}

IStageMonitorSessionManager& FStageMonitorModule::GetStageMonitorSessionManager()
{
	return *SessionManager;
}

void FStageMonitorModule::EnableMonitor(bool bEnable)
{
	if (bEnable)
	{
		StageMonitor->Start();
	}
	else
	{
		StageMonitor->Stop();
	}
}

IMPLEMENT_MODULE(FStageMonitorModule, StageMonitor)

