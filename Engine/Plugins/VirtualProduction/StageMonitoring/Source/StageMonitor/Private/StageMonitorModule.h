// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStageMonitorModule.h"


class FAutoConsoleCommand;
class FStageMonitor;
class FStageMonitorSessionManager;

DECLARE_LOG_CATEGORY_EXTERN(LogStageMonitor, Log, All);


class FStageMonitorModule : public IStageMonitorModule
{
public:	
	virtual ~FStageMonitorModule() = default;
	
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~End IModuleInterface

	//~Begin IStageMonitorModule interface
	virtual IStageMonitor& GetStageMonitor() override;
	virtual IStageMonitorSessionManager& GetStageMonitorSessionManager() override;
	virtual void EnableMonitor(bool bEnable) override;
	//~End IStageMonitorModule interface

private:

	/** Create StageMonitor when engine is fully initialized */
	void OnEngineLoopInitComplete();

protected:
	
	/** Single instance of the StageMonitor */
	TUniquePtr<FStageMonitor> StageMonitor;

	/** Stage monitor session manager responsible to create/load/save sessions */
	TUniquePtr<FStageMonitorSessionManager> SessionManager;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	TUniquePtr<FAutoConsoleCommand> CommandStart;
	TUniquePtr<FAutoConsoleCommand> CommandStop;
#endif
};
