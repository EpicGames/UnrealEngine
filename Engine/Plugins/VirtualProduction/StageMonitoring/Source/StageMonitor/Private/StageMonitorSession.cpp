// Copyright Epic Games, Inc. All Rights Reserved.

#include "StageMonitorSession.h"

#include "Misc/App.h"
#include "StageCriticalEventHandler.h"
#include "StageMonitorModule.h"
#include "UObject/StructOnScope.h"


FStageMonitorSession::FStageMonitorSession(const FString& InSessionName)
	: CriticalEventHandler(MakeUnique<FStageCriticalEventHandler>())
	, SessionName(InSessionName)
{

}

void FStageMonitorSession::HandleDiscoveredProvider(const FGuid& Identifier, const FStageInstanceDescriptor& Descriptor, const FMessageAddress& Address)
{
	//When a provider answers the discovery, it might 
	// -already known
	// -known in the passed but cleared
	// -unknown but matchable to previous instance
	
	const auto FindProviderLambda = [Identifier, &Descriptor](const FStageSessionProviderEntry& Other)
	{
		//Look for a matching lost/disconnected provider from the same machine + same stage
		if (Other.Descriptor.MachineName == Descriptor.MachineName && Other.Descriptor.FriendlyName == Descriptor.FriendlyName && Other.Descriptor.RolesStringified == Descriptor.RolesStringified)
		{
			UE_LOGF(LogStageMonitor, VeryVerbose, "Old provider recovered, %ls - (%ls (%d)) with roles '%ls' on sessionId %d"
				, *Descriptor.FriendlyName.ToString()
				, *Descriptor.MachineName
				, Descriptor.ProcessId
				, *Descriptor.RolesStringified
				, Descriptor.SessionId);

			return true;
		}

		return (Other.Identifier == Identifier);
	};

	//Verify if it's a cleared provider
	const int32 ClearedProviderIndex = ClearedProviders.IndexOfByPredicate(FindProviderLambda);
	if (ClearedProviderIndex != INDEX_NONE)
	{
		//If it was cleared, remove it from the cleared list and re-add it. 
		//Then, update the identifier mapping to be able to link old identifiers to new one
		const FGuid OldIdentifier = ClearedProviders[ClearedProviderIndex].Identifier;
		ClearedProviders.RemoveAtSwap(ClearedProviderIndex);
		AddProvider(Identifier, Descriptor, Address);
		UpdateIdentifierMapping(OldIdentifier, Identifier);
	}
	else
	{
		const int32 ExistingProviderIndex = Providers.IndexOfByPredicate(FindProviderLambda);
		if (ExistingProviderIndex != INDEX_NONE)
		{
			//In case of an existing provider, 
			FStageSessionProviderEntry& Entry = Providers[ExistingProviderIndex];
			if (Entry.State != EStageDataProviderState::Active)
			{
				//Need to update data about a closed provider. Its address will certainly have changed and descriptor could also have
				const FGuid PreviousIdentifier = Entry.Identifier;
				FillProviderDescription(Identifier, Descriptor, Address, Entry);

				if (Identifier != PreviousIdentifier)
				{
					//If identifier change, make sure our mapping is up to date
					UpdateIdentifierMapping(PreviousIdentifier, Identifier);

					//If we're reslotting an old provider to a new one (same stagename / same machine) trigger a list change to let listeners know about it
					OnStageDataProviderListChanged().Broadcast();
				}
			}
		}
		else
		{
			AddProvider(Identifier, Descriptor, Address);
		}
	}
}

bool FStageMonitorSession::AddProvider(const FGuid& Identifier, const FStageInstanceDescriptor& Descriptor, const FMessageAddress& Address)
{
	if (Providers.ContainsByPredicate([Identifier](const FStageSessionProviderEntry& Other) { return Other.Identifier == Identifier; }))
	{
		return false;
	}

	UE_LOGF(LogStageMonitor, VeryVerbose, "Adding provider  %ls - (%ls (%d)) with roles '%ls' on sessionId %d"
		, *Descriptor.FriendlyName.ToString()
		, *Descriptor.MachineName
		, Descriptor.ProcessId
		, *Descriptor.RolesStringified
		, Descriptor.SessionId);
	
	FStageSessionProviderEntry& NewEntry = Providers.Emplace_GetRef();
	FillProviderDescription(Identifier, Descriptor, Address, NewEntry);

	//Let know a new provider was added to the list
	OnStageDataProviderListChanged().Broadcast();

	return true;
}

void FStageMonitorSession::FillProviderDescription(const FGuid& Identifier, const FStageInstanceDescriptor& NewDescriptor, const FMessageAddress& NewAddress, FStageSessionProviderEntry& OutEntry)
{
	OutEntry.Identifier = Identifier;
	OutEntry.Descriptor = NewDescriptor;
	OutEntry.Address = NewAddress;
	OutEntry.State = EStageDataProviderState::Active;
	OutEntry.LastReceivedMessageTime = FApp::GetCurrentTime();
}

void FStageMonitorSession::UpdateIdentifierMapping(const FGuid& OldIdentifier, const FGuid& NewIdentifier)
{
	//Go over old entries to update the value to the latest identifier an old one now points to
	for (TPair<FGuid, FGuid>& Entry : IdentifierMapping)
	{
		if (Entry.Value == OldIdentifier)
		{
			Entry.Value = NewIdentifier;
		}
	}

	IdentifierMapping.FindOrAdd(OldIdentifier) = NewIdentifier;
}

void FStageMonitorSession::AddProviderMessage(UScriptStruct* Type, const FStageProviderMessage* MessageData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStageMonitorSession::AddProviderMessage);

	if (MessageData == nullptr)
	{
		return;
	}

	//Only process messages coming from registered machines
	FStageSessionProviderEntry* Provider = Providers.FindByPredicate([MessageData](const FStageSessionProviderEntry& Other) { return MessageData->Identifier == Other.Identifier; });
	if (Provider == nullptr)
	{
		//Provider might have been cleared in the meantime and mapping needs to be used
		if (IdentifierMapping.Contains(MessageData->Identifier))
		{
			const FGuid MappedIdentifier = IdentifierMapping[MessageData->Identifier];
			Provider = Providers.FindByPredicate([MappedIdentifier](const FStageSessionProviderEntry& Other) { return MappedIdentifier == Other.Identifier; });
		}
	}

	if (Provider == nullptr)
	{
		return;
	}

	//Stamp last time this provider received a message
	Provider->LastReceivedMessageTime = FApp::GetCurrentTime();
	
	//If we're dealing with discovery response, just stamp received time and exit
	if (Type == FStageProviderDiscoveryResponseMessage::StaticStruct())
	{
		return;
	}

	// Special handling for critical event to track stage state
	if (Type->IsChildOf(FCriticalStateProviderMessage::StaticStruct()))
	{
		CriticalEventHandler->HandleCriticalEventMessage(static_cast<const FCriticalStateProviderMessage*>(MessageData));
	}

	//Add this message to session data => Full entry list and per provider / data type list
	TSharedPtr<FStageDataEntry> NewEntry = MakeShared<FStageDataEntry>();
	NewEntry->MessageTime = FApp::GetCurrentTime();
	NewEntry->Data = MakeShared<FStructOnScope>(Type);
	Type->CopyScriptStruct(NewEntry->Data->GetStructMemory(), MessageData);

	UpdateProviderLatestEntry(MessageData->Identifier, Type, NewEntry);
	InsertNewEntry(NewEntry);
	
	//Let know listeners that new data was received
	OnStageSessionNewDataReceived().Broadcast(NewEntry);
}

TArray<FMessageAddress> FStageMonitorSession::GetProvidersAddress() const
{
	TArray<FMessageAddress> Addresses;
	for (const FStageSessionProviderEntry& Entry : Providers)
	{
		if (Entry.State != EStageDataProviderState::Closed)
		{
			Addresses.Add(Entry.Address);
		}
	}

	return Addresses;
}

void FStageMonitorSession::SetProviderState(const FGuid& Identifier, EStageDataProviderState State)
{
	if (FStageSessionProviderEntry* Provider = Providers.FindByPredicate([Identifier](const FStageSessionProviderEntry& Other) { return Other.Identifier == Identifier; }))
	{
		if (State != Provider->State)
		{
			OnStageDataProviderStateChanged().Broadcast(Identifier, State);
		}

		Provider->State = State;
	}
}

bool FStageMonitorSession::GetProvider(const FGuid& Identifier, FStageSessionProviderEntry& OutProviderEntry, EGetProviderFlags  Flags) const
{
	FGuid UsedIdentifier = Identifier;
	if (EnumHasAnyFlags(Flags, EGetProviderFlags::UseIdentifierMapping) && IdentifierMapping.Contains(Identifier))
	{
		UsedIdentifier = IdentifierMapping[Identifier];
	}

	if (const FStageSessionProviderEntry* Provider = Providers.FindByPredicate([UsedIdentifier](const FStageSessionProviderEntry& Other) { return Other.Identifier == UsedIdentifier; }))
	{
		OutProviderEntry = *Provider;
		return true;
	}

	if (EnumHasAnyFlags(Flags, EGetProviderFlags::UseClearedProviders))
	{
		if (const FStageSessionProviderEntry* Provider = ClearedProviders.FindByPredicate([UsedIdentifier](const FStageSessionProviderEntry& Other) { return Other.Identifier == UsedIdentifier; }))
		{
			OutProviderEntry = *Provider;
			return true;
		}
	}

	return false;
}

void FStageMonitorSession::ClearAll()
{
	//Clear all entry list and per provider latest data
	Entries.Empty();
	ProviderLatestData.Empty();
	
	//Let know our listeners that we have just been cleared
	OnStageSessionDataClearedDelegate.Broadcast();
}

void FStageMonitorSession::ClearUnresponsiveProviders()
{
	for (auto Iter = Providers.CreateIterator(); Iter; Iter++)
	{
		if (Iter->State != EStageDataProviderState::Active)
		{
			ClearedProviders.Emplace(MoveTemp(*Iter));
			Iter.RemoveCurrent();
		}
	}

	OnStageDataProviderListChanged().Broadcast();
}

TSharedPtr<FStageDataEntry> FStageMonitorSession::GetLatest(const FGuid& Identifier, UScriptStruct* Type)
{
	if (ProviderLatestData.Contains(Identifier))
	{
		//Find the provider and see if it has an entry for this data type.
		TArray<TSharedPtr<FStageDataEntry>>& ProviderEntries = ProviderLatestData[Identifier];
		TSharedPtr<FStageDataEntry>* Latest = ProviderEntries.FindByPredicate([Type](const TSharedPtr<FStageDataEntry>& Other)
			{
				if (Other.IsValid())
				{
					if (Other->Data.IsValid())
					{
						return Other->Data->GetStruct() == Type;
					}
				}

				return false;
			});

		if (Latest != nullptr)
		{
			return *Latest;
		}
	}

	return TSharedPtr<FStageDataEntry>();
}

EStageDataProviderState FStageMonitorSession::GetProviderState(const FGuid& Identifier)
{
	const FStageSessionProviderEntry* Provider = Providers.FindByPredicate([Identifier](const FStageSessionProviderEntry& Other) { return Other.Identifier == Identifier; });
	if (Provider)
	{
		return Provider->State;
	}
	
	//Unknown provider state defaults to closed
	return EStageDataProviderState::Closed;
}

void FStageMonitorSession::UpdateProviderLatestEntry(const FGuid& Identifier, UScriptStruct* Type, TSharedPtr<FStageDataEntry> NewEntry)
{
	//Update latest entry of that type
	TArray<TSharedPtr<FStageDataEntry>>& ProviderEntries = ProviderLatestData.FindOrAdd(Identifier);
	TSharedPtr<FStageDataEntry>* Latest = ProviderEntries.FindByPredicate([Type](const TSharedPtr<FStageDataEntry>& Other)
		{
			if (Other.IsValid())
			{
				if (Other->Data.IsValid())
				{
					return Other->Data->GetStruct() == Type;
				}
			}

			return false;
		});

	if (Latest == nullptr)
	{
		ProviderEntries.Add(NewEntry);
	}
	else
	{
		*Latest = NewEntry;
	}
}

void FStageMonitorSession::InsertNewEntry(TSharedPtr<FStageDataEntry> NewEntry)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStageMonitorSession::InsertNewEntry);

	const FStageProviderMessage* Message = reinterpret_cast<const FStageProviderMessage*>(NewEntry->Data->GetStructMemory());
	check(Message);

	//We are always inserting in order so we shouldn't have to navigate in our list much
	const double NewFrameSeconds = Message->FrameTime.AsSeconds();
	const double NewFrameDay = FMath::Floor(Message->DateTime.GetJulianDay());

	if (NewFrameSeconds >= 0)
	{
		int32 EntryIndex = Entries.Num() - 1;
		for (; EntryIndex >= 0; --EntryIndex)
		{
			const FStageProviderMessage* ThisEntry = reinterpret_cast<const FStageProviderMessage*>(Entries[EntryIndex]->Data->GetStructMemory());
		
			const double ThisFrameSeconds = ThisEntry->FrameTime.AsSeconds();
			const double ThisFrameDay = FMath::Floor(ThisEntry->DateTime.GetJulianDay());

			if (ThisFrameSeconds <= NewFrameSeconds || NewFrameDay > ThisFrameDay)
			{
				break;
			}
		}

		const int32 InsertIndex = EntryIndex + 1;
		Entries.Insert(NewEntry, InsertIndex);
	}
	else
	{
		// Invalid entries with negative timecode get added to the end of the list to avoid going through all the list when they're inserted.
		Entries.Add(NewEntry);
	}
}

void FStageMonitorSession::GetAllEntries(TArray<TSharedPtr<FStageDataEntry>>& OutEntries)
{
	OutEntries = Entries; 
}

bool FStageMonitorSession::IsStageInCriticalState() const
{
	return CriticalEventHandler->IsCriticalStateActive();
}

bool FStageMonitorSession::IsTimePartOfCriticalState(double TimeInSeconds) const
{
	return CriticalEventHandler->IsTimingPartOfCriticalRange(TimeInSeconds);
}

FName FStageMonitorSession::GetCurrentCriticalStateSource() const
{
	return CriticalEventHandler->GetCurrentCriticalStateSource();
}

TArray<FName> FStageMonitorSession::GetCriticalStateHistorySources() const
{
	return CriticalEventHandler->GetCriticalStateHistorySources();
}

TArray<FName> FStageMonitorSession::GetCriticalStateSources(double TimeInSeconds) const
{
	return CriticalEventHandler->GetCriticalStateSources(TimeInSeconds);
}

FString FStageMonitorSession::GetSessionName() const
{
	return SessionName;
}

void FStageMonitorSession::GetIdentifierMapping(TMap<FGuid, FGuid>& OutMapping)
{
	OutMapping = IdentifierMapping;
}

void FStageMonitorSession::SetIdentifierMapping(const TMap<FGuid, FGuid>& NewMapping)
{
	IdentifierMapping = NewMapping;
}
