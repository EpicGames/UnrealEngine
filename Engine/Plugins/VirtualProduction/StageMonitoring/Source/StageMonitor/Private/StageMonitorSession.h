// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStageMonitorSession.h"


class FStageCriticalEventHandler;
enum class EStageDataProviderState : uint8;
struct FMessageAddress;
struct FStageInstanceDescriptor;
struct FStageProviderMessage;



/**
 * Implementation of the stage monitor session. Handles new data being received and organizes it for outside access
 */
class FStageMonitorSession : public IStageMonitorSession
{
public:
	FStageMonitorSession(const FString& InSessionName);
	~FStageMonitorSession() = default;
public:

	
	//~Begin IStageMonitorSession interface
	virtual void HandleDiscoveredProvider(const FGuid& Identifier, const FStageInstanceDescriptor& Descriptor, const FMessageAddress& Address) override;
	virtual bool AddProvider(const FGuid& Identifier, const FStageInstanceDescriptor& Descriptor, const FMessageAddress& Address) override;
	virtual void AddProviderMessage(UScriptStruct* Type, const FStageProviderMessage* MessageData) override;
	virtual TArray<FMessageAddress> GetProvidersAddress() const override;
	virtual void SetProviderState(const FGuid& Identifier, EStageDataProviderState State) override;
	virtual TConstArrayView<FStageSessionProviderEntry> GetProviders() const override { return MakeArrayView(Providers); }
	virtual TConstArrayView<FStageSessionProviderEntry> GetClearedProviders() const override { return MakeArrayView(ClearedProviders); }
	virtual bool GetProvider(const FGuid& Identifier, FStageSessionProviderEntry& OutProviderEntry, EGetProviderFlags  Flags = EGetProviderFlags::Default) const override;
	virtual void ClearAll() override;
	virtual void ClearUnresponsiveProviders() override;	
	virtual void GetAllEntries(TArray<TSharedPtr<FStageDataEntry>>& OutEntries) override;
	virtual TSharedPtr<FStageDataEntry> GetLatest(const FGuid& Identifier, UScriptStruct* Type) override;
	virtual EStageDataProviderState GetProviderState(const FGuid& Identifier) override;
	virtual bool IsStageInCriticalState() const override;
	virtual bool IsTimePartOfCriticalState(double TimeInSeconds) const override;
	virtual FName GetCurrentCriticalStateSource() const override;
	virtual TArray<FName> GetCriticalStateHistorySources() const override;
	virtual TArray<FName> GetCriticalStateSources(double TimeInSeconds) const override;
	virtual FString GetSessionName() const override;

	virtual FOnStageSessionNewDataReceived& OnStageSessionNewDataReceived() override { return OnNewDataReceivedDelegate; }
	virtual FOnStageSessionDataCleared& OnStageSessionDataCleared() override { return OnStageSessionDataClearedDelegate; }
	virtual FOnStageDataProviderListChanged& OnStageDataProviderListChanged() override { return OnStageDataProviderListChangedDelegate; }
	virtual FOnStageDataProviderStateChanged& OnStageDataProviderStateChanged() override { return OnStageDataProviderStateChangedDelegate; }
	//~End IStageMonitorSession interface

	/** Retrieve the current mapping for identifier */
	void GetIdentifierMapping(TMap<FGuid, FGuid>& OutMapping);

	/** Sets a new identifier mapping for the session */
	void SetIdentifierMapping(const TMap<FGuid, FGuid>& NewMapping);

private:

	/** Update snapshot for this message type per provider. */
	void UpdateProviderLatestEntry(const FGuid& Identifier, UScriptStruct* Type, TSharedPtr<FStageDataEntry> NewEntry);

	/** Inserts a new message entry in our list, sorted by timecode in seconds */
	void InsertNewEntry(TSharedPtr<FStageDataEntry> NewEntry);

	/** Fills in the information on a data provider entry
	 * Could be for a new entry, an existing entry that we are updating since it was rediscovered
	 */
	void FillProviderDescription(const FGuid& Identifier, const FStageInstanceDescriptor& NewDescriptor, const FMessageAddress& NewAddress, FStageSessionProviderEntry& OutEntry);

	/**
	 * Updates mapping for identifier that existed in the past to the latest identifier associated
	 * with the "same" provider. When showing old activities, this must be kept to associate
	 * old activity identifier to active provider's identifier
	 */
	void UpdateIdentifierMapping(const FGuid& OldIdentifier, const FGuid& NewIdentifier);

private:
	
	/** List of providers currently monitored */
	TArray<FStageSessionProviderEntry> Providers;

	/** List of all messages received. */
	TArray<TSharedPtr<FStageDataEntry>> Entries;

	/** Latest entry per message type for each provider */
	TMap<FGuid, TArray<TSharedPtr<FStageDataEntry>>> ProviderLatestData;

	/** Manages critical state messages to manage this session's state */
	TUniquePtr<FStageCriticalEventHandler> CriticalEventHandler;

	/** This session name. Used for display */
	FString SessionName;

	/** Delegate triggered when new data was received */
	FOnStageSessionNewDataReceived OnNewDataReceivedDelegate;

	/** Delegate triggered when session data is cleared */
	FOnStageSessionDataCleared OnStageSessionDataClearedDelegate;

	/** Delegate triggered when the list of monitored providers changed */
	FOnStageDataProviderListChanged OnStageDataProviderListChangedDelegate;

	/** Delegate triggered when the state of a monitored providers changed */
	FOnStageDataProviderStateChanged OnStageDataProviderStateChangedDelegate;

	/** Unresponsive providers cleared from the list */
	TArray<FStageSessionProviderEntry> ClearedProviders;

	/** 
	 * When a provider is rematched to an old one, its identifier might have changed
	 * Old activities will still have the previous identifier, so this is used to keep
	 * track of identifier history
	 */
	TMap<FGuid, FGuid> IdentifierMapping;
};

