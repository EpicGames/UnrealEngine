// Copyright Epic Games, Inc. All Rights Reserved.

#include "StageMonitorSessionManager.h"

#include "Async/Async.h"
#include "HAL/FileManager.h"
#include "JsonObjectConverter.h"
#include "Misc/Paths.h"
#include "StageCriticalEventHandler.h"
#include "StageMonitorModule.h"
#include "StageMonitorSession.h"
#include "UObject/StructOnScope.h"



TSharedPtr<IStageMonitorSession> FStageMonitorSessionManager::CreateSession()
{
	if (!ActiveSession.IsValid())
	{
		const FString LiveSessionName = TEXT("Live");
		ActiveSession = MakeShared<FStageMonitorSession>(LiveSessionName);
		return ActiveSession;
	}

	return TSharedPtr<IStageMonitorSession>();
}

bool FStageMonitorSessionManager::LoadSession(const FString& FileName)
{
	//Clear out the loaded session. If it fails, loaded one is empty and UI will show nothing
	LoadedSession.Reset();

	if (!bIsLoading)
	{
		bIsLoading = true;

		//Schedule task to load from file on background thread
		//UScriptStructs are looked using FindObjectSafe. UObjects shouldn't be used 
		//on other threads than gamethread.
		AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [this, FileName]()
		{
			LoadSessionTask_AnyThread(FileName);
		});

		return true;
	}

	return false;
}

bool FStageMonitorSessionManager::SaveSession(const FString& FileName)
{
	if (ActiveSession.IsValid() && !bIsSaving)
	{
		bIsSaving = true;

		//Setup task data 
		FSavingTaskData TaskData;
		TaskData.Providers = ActiveSession->GetProviders();
		ActiveSession->GetIdentifierMapping(TaskData.IdentifierMapping);
		ActiveSession->GetAllEntries(TaskData.Entries);
		TaskData.Settings = GetDefault<UStageMonitoringSettings>()->ExportSettings;

		//Schedule task to save to file on background thread
		//UScriptStructs are used to serialize to json. 
		AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [this, FileName, TaskData = MoveTemp(TaskData)]()
		{
			SaveSessionTask_AnyThread(FileName, TaskData);
		});

		return true;
	}

	return false;
}

TSharedPtr<IStageMonitorSession> FStageMonitorSessionManager::GetActiveSession()
{
	return ActiveSession;
}

TSharedPtr<IStageMonitorSession> FStageMonitorSessionManager::GetLoadedSession()
{
	return LoadedSession;
}

IStageMonitorSessionManager::FOnStageMonitorSessionLoaded& FStageMonitorSessionManager::OnStageMonitorSessionLoaded()
{
	return OnStageMonitorSessionLoadedDelegate;
}

IStageMonitorSessionManager::FOnStageMonitorSessionSaved& FStageMonitorSessionManager::OnStageMonitorSessionSaved()
{
	return OnStageMonitorSessionSavedDelegate;
}

UScriptStruct* FStageMonitorSessionManager::FindStructFromName_AnyThread(FName TypeName)
{
	//Use previously loaded one if part of the cache
	if (TypeCache.Contains(TypeName))
	{
		return TypeCache[TypeName];
	}

	const bool bExactClass = true;
	UScriptStruct* Struct = FindFirstObjectSafe<UScriptStruct>(*TypeName.ToString(), bExactClass ? EFindFirstObjectOptions::ExactClass : EFindFirstObjectOptions::None);
	if (Struct)
	{
		TypeCache.FindOrAdd(TypeName) = Struct;
	}

	return Struct;
}

void FStageMonitorSessionManager::OnLoadSessionCompletedTask(const TSharedPtr<FStageMonitorSession>& InLoadedSession, const FString& FileName, const FString& InError)
{
	bool bWasSuccessful = true;
	if (InError.IsEmpty())
	{
		UE_LOGF(LogStageMonitor, Log, "Loaded Stage Session from '%ls'", *FileName);
		LoadedSession = InLoadedSession;
	}
	else
	{
		UE_LOGF(LogStageMonitor, Error, "Error loading file '%ls': %ls", *FileName, *InError)
		bWasSuccessful = false;
	}

	bIsLoading = false;
	OnStageMonitorSessionLoadedDelegate.Broadcast();
}

void FStageMonitorSessionManager::OnSaveSessionCompletedTask(const FString& FileName, const FString& InError)
{
	bool bWasSuccessful = true;
	if (InError.IsEmpty())
	{
		UE_LOGF(LogStageMonitor, Log, "Saving current stage session to file '%ls' was successful", *FileName);
	}
	else
	{
		UE_LOGF(LogStageMonitor, Error, "Saving file '%ls' failed: %ls", *FileName, *InError);
		bWasSuccessful = false;
	}

	bIsSaving = false;
	OnStageMonitorSessionSavedDelegate.Broadcast();
}

void FStageMonitorSessionManager::LoadSessionTask_AnyThread(const FString& FileName)
{
	TSharedPtr<FStageMonitorSession> ImportedSession;
	FString Error;
	if (TUniquePtr<FArchive> FileReader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*FileName)))
	{
		TSharedRef< TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(FileReader.Get());
		TSharedPtr<FJsonObject> RootData = MakeShared<FJsonObject>();
		if (FJsonSerializer::Deserialize(JsonReader, RootData))
		{
			FMonitorSessionInfo Info;
			if (FJsonObjectConverter::JsonObjectToUStruct(RootData.ToSharedRef(), &Info))
			{
				if (Info.CurrentVersion == FMonitorSessionInfo::CurrentVersion)
				{
					ImportedSession = MakeShared<FStageMonitorSession>(FPaths::GetBaseFilename(FileName));


					//Identifier mapping load
					const TSharedPtr<FJsonObject>& MappingObject = RootData->GetObjectField(TEXT("IdentifierMapping"));
					if (MappingObject.IsValid())
					{
						const int32 MapSize = MappingObject->Values.Num();
						TMap<FGuid, FGuid> Mapping;
						Mapping.Reserve(MapSize);

						for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Entry : MappingObject->Values)
						{
							if (Entry.Value.IsValid() && !Entry.Value->IsNull())
							{
								const FGuid Key(FString(Entry.Key));
								FString IdentifierValue;
								if (Entry.Value->TryGetString(IdentifierValue))
								{
									Mapping.FindOrAdd(Key) = FGuid(IdentifierValue);
								}
							}
						}
					
						ImportedSession->SetIdentifierMapping(Mapping);
					}


					const TArray<TSharedPtr<FJsonValue>>& Providers = RootData->GetArrayField(TEXT("Providers"));
					for (const TSharedPtr<FJsonValue>& Provider : Providers)
					{
						const TSharedPtr<FJsonObject>* ProviderObject = nullptr;
						FStageSessionProviderEntry NewProvider;
						if (FJsonObjectConverter::JsonObjectToUStruct<FStageSessionProviderEntry>(Provider->AsObject().ToSharedRef(), &NewProvider))
						{
							ImportedSession->AddProvider(NewProvider.Identifier, NewProvider.Descriptor, FMessageAddress::NewAddress());
						}
						else
						{
							UE_LOGF(LogStageMonitor, Warning, "Could not convert JsonObject to Stage Session Provider structure");
						}
					}

					const TArray<TSharedPtr<FJsonValue>>& Entries = RootData->GetArrayField(TEXT("Entries"));
					for (const TSharedPtr<FJsonValue>& Entry : Entries)
					{
						const TSharedPtr<FJsonObject> EntryObject = Entry->AsObject();
						if (EntryObject)
						{
							const FString EntryType = EntryObject->GetStringField(TEXT("EntryType"));
							UScriptStruct* Struct = FindStructFromName_AnyThread(*EntryType);
							if (Struct)
							{
								FStructOnScope NewData(Struct);
								if (FJsonObjectConverter::JsonObjectToUStruct(EntryObject.ToSharedRef(), NewData.GetStruct(), NewData.GetStructMemory()))
								{
									ImportedSession->AddProviderMessage(Struct, reinterpret_cast<FStageProviderMessage*>(NewData.GetStructMemory()));
								}
								else
								{
									UE_LOGF(LogStageMonitor, Warning, "Could not convert JsonObject to StageSessionEntry of type '%ls'", *EntryType);
								}
							}
							else
							{
								UE_LOGF(LogStageMonitor, Warning, "Could not find UStruct of type '%ls'", *EntryType);
							}
						}
					}
				}
				else
				{
					Error = FString::Printf(TEXT("Can't read stage session file with version %d."), Info.Version);
				}
			}
			else
			{
				Error = TEXT("Could not read stage session header.");
			}
		}
		else
		{
			Error = TEXT("Could not deserialize to json data.");
		}

		FileReader->Close();
	}
	else
	{
		Error = TEXT("Couldn't read file.");
	}

	//Loading done, schedule completion on gamethread to be able to trigger the completion delegate thread safely 
	AsyncTask(ENamedThreads::GameThread, [this, ImportedSession = MoveTemp(ImportedSession), FileName, Error = MoveTemp(Error)]()
	{
		OnLoadSessionCompletedTask(ImportedSession, FileName, Error);
	});
}

void FStageMonitorSessionManager::SaveSessionTask_AnyThread(const FString& FileName, const FSavingTaskData& TaskData)
{
	FString Error;
	if (TUniquePtr<FArchive> FileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*FileName)))
	{
		TSharedRef< TJsonWriter<> > JsonWriter = TJsonWriterFactory<>::Create(FileWriter.Get());
		TSharedPtr<FJsonObject> RootData = MakeShared<FJsonObject>();

		//Start file with stage session info header
		FMonitorSessionInfo Info;
		if (FJsonObjectConverter::UStructToJsonObject(FMonitorSessionInfo::StaticStruct(), &Info, RootData.ToSharedRef(), 0 /*Checkflags*/, 0/*Skipflags*/))
		{
			//Provider identifier mapping first
			TSharedPtr<FJsonObject> MappingObject = MakeShared<FJsonObject>();
			for (const TPair<FGuid, FGuid>& Mapping : TaskData.IdentifierMapping)
			{
				const FString Key = Mapping.Key.ToString();
				TSharedPtr<FJsonValueString> JSonValue = MakeShared<FJsonValueString>(Mapping.Value.ToString());
				MappingObject->SetField(Key, JSonValue);
			}
			RootData->SetObjectField(TEXT("IdentifierMapping"), MappingObject);

			//Current Providers
			TArray<TSharedPtr<FJsonValue>> ProvidersArray;
			ProvidersArray.Reserve(TaskData.Providers.Num());

			//Jsonify provider list. One object per provider with indexed name
			int32 ProviderIndex = 0;
			for (const FStageSessionProviderEntry& Provider : TaskData.Providers)
			{
				TSharedPtr<FJsonObject> ProviderObject = MakeShared<FJsonObject>();
				if (FJsonObjectConverter::UStructToJsonObject(FStageSessionProviderEntry::StaticStruct(), &Provider, ProviderObject.ToSharedRef(), 0 /*Checkflags*/, 0/*Skipflags*/))
				{
					ProvidersArray.Add(MakeShared<FJsonValueObject>(ProviderObject));
				}
				else
				{
					UE_LOGF(LogStageMonitor, Warning, "Failed to convert provider ('%ls) struct to Json", *Provider.Descriptor.FriendlyName.ToString());
				}
			}

			RootData->SetArrayField(TEXT("Providers"), ProvidersArray);

			TArray<TSharedPtr<FJsonValue>> EntriesArray;
			EntriesArray.Reserve(TaskData.Entries.Num());

			//Last entry for each period message type in case settings wants only latest instance
			TMap<FName, TSharedPtr<FStageDataEntry>> LastPeriodicEntry;

			//Jsonify entries list. One object per entry with indexed name. Structure typename will be added before the entry object
			int32 EntryIndex = 0;
			for (const TSharedPtr<FStageDataEntry>& Entry : TaskData.Entries)
			{
				if (Entry.IsValid() && Entry->Data.IsValid())
				{
					check(Entry->Data->GetStruct()->IsChildOf(FStageProviderMessage::StaticStruct()));
					const FName EntryType = Entry->Data->GetStruct()->GetFName();

					//Verify if that message type is excluded or not
					if (!TaskData.Settings.ExcludedMessageTypes.ContainsByPredicate([EntryType](const FStageMessageTypeWrapper& Other) { return Other.MessageType == EntryType; }))
					{
						bool bCanAddEntry = true;
						if (TaskData.Settings.bKeepOnlyLastPeriodMessage)
						{
							if (Entry->Data->GetStruct()->IsChildOf(FStageProviderPeriodicMessage::StaticStruct()))
							{
								LastPeriodicEntry.FindOrAdd(EntryType) = Entry;
								bCanAddEntry = false;
							}
						}

						if (bCanAddEntry)
						{
							TSharedPtr<FJsonObject> EntryObject = MakeShared<FJsonObject>();
							EntryObject->SetStringField(FString(TEXT("EntryType")), EntryType.ToString());
							if (FJsonObjectConverter::UStructToJsonObject(Entry->Data->GetStruct(), Entry->Data->GetStructMemory(), EntryObject.ToSharedRef(), 0 /*Checkflags*/, 0/*Skipflags*/))
							{
								EntriesArray.Add(MakeShared<FJsonValueObject>(EntryObject));
							}
							else
							{
								UE_LOGF(LogStageMonitor, Warning, "Failed to convert session entry of type '%ls' to Json", *EntryType.ToString());
							}
						}
					}
				}
			}

			//Now write out last period messages entries
			for (const TPair<FName, TSharedPtr<FStageDataEntry>>& PeriodEntry : LastPeriodicEntry)
			{
				const FName EntryType = PeriodEntry.Key;
				const TSharedPtr<FStageDataEntry>& Entry = PeriodEntry.Value;

				TSharedPtr<FJsonObject> EntryObject = MakeShared<FJsonObject>();
				EntryObject->SetStringField(FString(TEXT("EntryType")), EntryType.ToString());
				if (FJsonObjectConverter::UStructToJsonObject(Entry->Data->GetStruct(), Entry->Data->GetStructMemory(), EntryObject.ToSharedRef(), 0 /*Checkflags*/, 0/*Skipflags*/))
				{
					EntriesArray.Add(MakeShared<FJsonValueObject>(EntryObject));
				}
				else
				{
					UE_LOGF(LogStageMonitor, Warning, "Failed to convert session entry of type '%ls' to Json", *EntryType.ToString());
				}
			}

			//Write out entries array
			RootData->SetArrayField(TEXT("Entries"), EntriesArray);

			if (!FJsonSerializer::Serialize(RootData.ToSharedRef(), JsonWriter))
			{
				Error = TEXT("Could not serialize data to json.");
			}
		}
		else
		{
			Error = TEXT("Could not convert MonitorSessionInfo to json.");
		}

		FileWriter->Close();
	}
	else
	{
		Error = TEXT("Could not create file.");
	}

	//Saving done, schedule completion on gamethread to be able to trigger the completion delegate thread safely 
	AsyncTask(ENamedThreads::GameThread, [this, FileName, Error = MoveTemp(Error)]()
	{
		OnSaveSessionCompletedTask(FileName, Error);
	});
}
