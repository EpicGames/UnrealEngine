// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStageMonitorSessionManager.h"

#include "StageMonitoringSettings.h"

#include "StageMonitorSessionManager.generated.h"

struct FStageDataEntry;
struct FStageSessionProviderEntry;

class FStageMonitorSession;

/**
 * Header written at beginning of json file to have an idea of what's in the file
 */
USTRUCT()
struct FMonitorSessionInfo
{
	GENERATED_BODY()

public:
	static constexpr int32 CurrentVersion = 1;
	UPROPERTY()
	int32 Version = CurrentVersion;
};


class STAGEMONITOR_API FStageMonitorSessionManager : public IStageMonitorSessionManager
{
public:
	//~ Begin IStageMonitorSessionManager interface
	virtual TSharedPtr<IStageMonitorSession> CreateSession() override;
	virtual bool LoadSession(const FString& FileName) override;
	virtual bool SaveSession(const FString& FileName) override;
	virtual TSharedPtr<IStageMonitorSession> GetActiveSession() override;
	virtual TSharedPtr<IStageMonitorSession> GetLoadedSession() override;
	virtual FOnStageMonitorSessionLoaded& OnStageMonitorSessionLoaded() override;
	virtual FOnStageMonitorSessionSaved& OnStageMonitorSessionSaved() override;
	//~End IStageMonitorSessionManager interface

private:

	/** Internal struct meant to pass data from save request to async task */
	struct FSavingTaskData
	{
		FStageDataExportSettings Settings;
		TArray<FStageSessionProviderEntry> Providers;
		TMap<FGuid, FGuid> IdentifierMapping;
		TArray<TSharedPtr<FStageDataEntry>> Entries;
	};

	/** Scheduled on gamethread by the anythread async task when loading was completed */
	void OnLoadSessionCompletedTask(const TSharedPtr<FStageMonitorSession>& InLoadedSession, const FString& FileName, const FString& InError);

	/** Scheduled on gamethread by the anythread async task when saving was completed */
	void OnSaveSessionCompletedTask(const FString& FileName, const FString& InError);

	/** Task used to load session from file and populate it from the json data */
	void LoadSessionTask_AnyThread(const FString& FileName);

	/** Task used to save current session to a file */
	void SaveSessionTask_AnyThread(const FString& FileName, const FSavingTaskData& TaskData);

	/** Finds a static struct from the specified FName. */
	UScriptStruct* FindStructFromName_AnyThread(FName TypeName);

private:

	/** Live session. Used by the monitor to dump new activities */
	TSharedPtr<FStageMonitorSession> ActiveSession;

	/** Imported session used to be displayed in UI */
	TSharedPtr<FStageMonitorSession> LoadedSession;

	/** Cached static structs per FName to accelerate looking for message type */
	TMap<FName, UScriptStruct*> TypeCache;

	/** Whether we are saving. Flagged before launching the async task and cleared on completion. Always on gamethread */
	bool bIsSaving = false;

	/** Whether we are loading. Flagged before launching the async task and cleared on completion. Always on gamethread */
	bool bIsLoading = false;

	/** Delegate triggered when the requested session was loaded */
	FOnStageMonitorSessionLoaded OnStageMonitorSessionLoadedDelegate;

	/** Delegate triggered when the requested session was loaded */
	FOnStageMonitorSessionSaved OnStageMonitorSessionSavedDelegate;
};
