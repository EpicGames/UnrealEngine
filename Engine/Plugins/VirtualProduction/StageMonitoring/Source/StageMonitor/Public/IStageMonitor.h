// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "StageMessages.h"
#include "UObject/StructOnScope.h"


class IStageMonitorSession;

/**
 * Interface for the stage monitor. Users can get the instance from the stage monitor module
 */
class STAGEMONITOR_API IStageMonitor
{
public:

	virtual ~IStageMonitor() {}
	
	/**
	 * Returns true if monitor is actively listening for activities
	 */
	virtual bool IsActive() const = 0;
};
