// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"


class IStageMonitor;
class IStageMonitorSessionManager;

class STAGEMONITOR_API IStageMonitorModule : public IModuleInterface
{
public:	
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IStageMonitorModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IStageMonitorModule>(ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() during shutdown if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	/**
	 * Returns the stage monitor instance
	 */
	virtual IStageMonitor& GetStageMonitor() = 0;

	/**
	 * Returns the stage monitor session manager
	 */
	virtual IStageMonitorSessionManager& GetStageMonitorSessionManager() = 0;

	/**
	 * Enable or disables StageMonitor
	 */
	virtual void EnableMonitor(bool bEnable) = 0;

private:
	
	static const FName ModuleName;
};
