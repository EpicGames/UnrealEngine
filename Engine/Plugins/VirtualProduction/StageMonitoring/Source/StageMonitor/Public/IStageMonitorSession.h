// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Delegates/DelegateCombinations.h"
#include "IMessageContext.h"
#include "StageMessages.h"
#include "UObject/StructOnScope.h"

#include "IStageMonitorSession.generated.h"

/**
 * Flags controlling the behavior of struct serializer backends.
 */
enum class EGetProviderFlags
{
	/**
	 * Nothing special
	 */
	None = 0,

	/**
	 * Include cleared provider in the search
	 * @note This is required to correctly support localization
	 */
	UseClearedProviders = 1 << 0,

	 /**
	  * Use Identifier mapping if Identifier isn't found in list.
	  */
	 UseIdentifierMapping = 1 << 1,

	/**
	 * Default.
	 */
	 Default = UseClearedProviders | UseIdentifierMapping,
};
ENUM_CLASS_FLAGS(EGetProviderFlags);

/**
 * Entry corresponding to a provider we are monitoring
 * Contains information related to the provider so we can communicate with it
 * and more dynamic information like last communication received
 */
 USTRUCT()
struct STAGEMONITOR_API FStageSessionProviderEntry
{
	GENERATED_BODY()

public:
	FStageSessionProviderEntry() = default;

	bool operator==(const FStageSessionProviderEntry& Other) const { return Identifier == Other.Identifier; }
	bool operator!=(const FStageSessionProviderEntry& Other) const { return !(*this == Other); }

	/** Identifier of this provider */
	UPROPERTY()
	FGuid Identifier;

	/** Detailed descriptor */
	UPROPERTY()
	FStageInstanceDescriptor Descriptor;

	/** Address of this provider */
	FMessageAddress Address;

	/** State of this provider based on message reception */
	EStageDataProviderState State = EStageDataProviderState::Closed;

	/** Timestamp when last message was received based on FApp::GetCurrentTime */
	double LastReceivedMessageTime = 0.0;
};

/**
 * Data entry containing data received from a provider
 */
struct STAGEMONITOR_API FStageDataEntry
{
	TSharedPtr<FStructOnScope> Data;
	double MessageTime = 0.0;
};

/**
 * Interface describing a session of data received by the monitor
 * Can be exported and reimported for future analysis
 */
class STAGEMONITOR_API IStageMonitorSession
{
public:

	virtual ~IStageMonitorSession() {}

	/**
	 * Handles discovery response of a provider. Might already be found, unresponsive or a new one
	 */
	virtual void HandleDiscoveredProvider(const FGuid& Identifier, const FStageInstanceDescriptor& Descriptor, const FMessageAddress& Address) = 0;

	/**
	 * Adds a new provider to the ones we're handling data for.
	 * Returns true if a provider was added
	 */
	virtual bool AddProvider(const FGuid& Identifier, const FStageInstanceDescriptor& Descriptor, const FMessageAddress& Address) = 0;

	/** Returns the addresses of the providers currently monitored */
	virtual TArray<FMessageAddress> GetProvidersAddress() const = 0;

	/** 
	 * Changes a provider's monitoring state 
	 */
	virtual void SetProviderState(const FGuid& Identifier, EStageDataProviderState State) = 0;

	/**
	 * Adds a message to the session
	 */
	virtual void AddProviderMessage(UScriptStruct* Type, const FStageProviderMessage* MessageData) = 0;

	/**
	 * Get all providers that have been connected to the monitor
	 */
	virtual TConstArrayView<FStageSessionProviderEntry> GetProviders() const = 0;

	/**
	 * Get providers that were connected at some point but got cleared
	 */
	virtual TConstArrayView<FStageSessionProviderEntry> GetClearedProviders() const = 0;

	/**
	 * Get ProviderEntry associated to an identifier
	 */
	virtual bool GetProvider(const FGuid& Identifier, FStageSessionProviderEntry& OutProviderEntry, EGetProviderFlags  Flags = EGetProviderFlags::Default) const = 0;

	/**
	 * Clear every activities of the session
	 */
	virtual void ClearAll() = 0;

	/**
	 * Clear unresponsive providers from active list
	 */
	virtual void ClearUnresponsiveProviders() = 0;

	/**
	 * Returns all entries that have been received
	 */
	virtual void GetAllEntries(TArray<TSharedPtr<FStageDataEntry>>& OutEntries) = 0;

	/**
	 * Returns the state of the desired provider
	 */
	virtual TSharedPtr<FStageDataEntry> GetLatest(const FGuid& Identifier, UScriptStruct* Type) = 0;

	/**
	 * Returns the state of the desired provider
	 */
	virtual EStageDataProviderState GetProviderState(const FGuid& Identifier) = 0;

	/**
	 * Returns true if stage monitor currently in critical state (i.e. recording)
	 */
	virtual bool IsStageInCriticalState() const = 0;

	/**
	 * Returns true if the given time is part of a critical state time range.
	 */
	virtual bool IsTimePartOfCriticalState(double TimeInSeconds) const = 0;

	/**
	 * Returns Source name of the current critical state. Returns None if not active
	 */
	virtual FName GetCurrentCriticalStateSource() const = 0;

	/**
	 * Returns a list of all sources that triggered a critical state
	 */
	virtual TArray<FName> GetCriticalStateHistorySources() const = 0;

	/**
	 * Returns a list of all sources that triggered a critical state during TimeInSeconds.
	 * If provided time is not part of a critical state, returned array will be empty
	 */
	virtual TArray<FName> GetCriticalStateSources(double TimeInSeconds) const = 0;

	/**
	 * Returns the name of that session.
	 */
	virtual FString GetSessionName() const = 0;


	/**
	 * Callback triggered when new data has been added to the session
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnStageSessionNewDataReceived, TSharedPtr<FStageDataEntry> /*Data*/);
	virtual FOnStageSessionNewDataReceived& OnStageSessionNewDataReceived() = 0;

	/**
	 * Callback triggered when data from the session has been cleared
	 */
	DECLARE_MULTICAST_DELEGATE(FOnStageSessionDataCleared);
	virtual FOnStageSessionDataCleared& OnStageSessionDataCleared() = 0;

	/**
     * Callback triggered when provider list changed
	 */
	DECLARE_MULTICAST_DELEGATE(FOnStageDataProviderListChanged);
	virtual FOnStageDataProviderListChanged& OnStageDataProviderListChanged() = 0;

	/**
	 * Callback triggered when provider state has changed
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnStageDataProviderStateChanged, const FGuid& /*Identifier*/, EStageDataProviderState /*NewState*/);
	virtual FOnStageDataProviderStateChanged& OnStageDataProviderStateChanged() = 0;
};
