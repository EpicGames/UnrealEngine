// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IStageMonitorSession;

class STAGEMONITOR_API IStageMonitorSessionManager
{
public:

	virtual ~IStageMonitorSessionManager() = default;

public:	
	/** Creates a new active session. If one is already present, it will return an invalid session */
	virtual TSharedPtr<IStageMonitorSession> CreateSession() = 0;

	/** 
	 * Request to load a session. Returns true if loading was effectively requested
	 * Loading will be done async and a callback. Listen for OnStageMonitorSessionLoaded to know when it's done
	 */
	virtual bool LoadSession(const FString& FileName) = 0;

	/**
	 * Request to save a session. Returns true if saving was effectively requested
	 * Loading will be done async and a callback. Listen for OnStageMonitorSessionSaved to know when it's done
	 */
	virtual bool SaveSession(const FString& FileName) = 0;

	/** Returns the current active (live) session */
	virtual TSharedPtr<IStageMonitorSession> GetActiveSession() = 0;

	/** Returns the loaded session */
	virtual TSharedPtr<IStageMonitorSession> GetLoadedSession() = 0;

	/**
	 * Callback triggered when a requested session was loaded
	 */
	DECLARE_MULTICAST_DELEGATE(FOnStageMonitorSessionLoaded);
	virtual FOnStageMonitorSessionLoaded& OnStageMonitorSessionLoaded() = 0;

	/**
	 * Callback triggered when a requested session was saved
	 */
	DECLARE_MULTICAST_DELEGATE(FOnStageMonitorSessionSaved);
	virtual FOnStageMonitorSessionSaved& OnStageMonitorSessionSaved() = 0;
};
