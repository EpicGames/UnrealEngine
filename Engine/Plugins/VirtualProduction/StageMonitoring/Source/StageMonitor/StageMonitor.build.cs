// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class StageMonitor : ModuleRules
{
	public StageMonitor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"StageDataCore",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"GameplayTags",
				"Json",
				"JsonUtilities",
				"Serialization",
				"StageMonitorCommon",
				"VPRoles",
				"VPUtilities",
				
			}
		);
	}
}
