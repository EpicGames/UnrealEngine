// Copyright Epic Games, Inc. All Rights Reserved.

#include "StageMonitorUtils.h"

#include "Engine/Engine.h"
#include "StageMonitoringSettings.h"
#include "VPRolesSubsystem.h"


namespace StageMonitorUtils
{
	static const FString CachedComputerName = FPlatformProcess::ComputerName();

	FStageInstanceDescriptor GetInstanceDescriptor()
	{
		FStageInstanceDescriptor Descriptor;

		//A machine could spawn multiple UE instances. Need to be able to differentiate them. ProcessId is there for that reason
		Descriptor.MachineName = CachedComputerName;
		Descriptor.ProcessId = FPlatformProcess::GetCurrentProcessId();

		if (GEngine)
		{
			if (const UVirtualProductionRolesSubsystem* VPRolesSubsystem = GEngine->GetEngineSubsystem<UVirtualProductionRolesSubsystem>())
			{
				Descriptor.RolesStringified = VPRolesSubsystem->GetActiveRolesString();
			}
		}

		Descriptor.FriendlyName = GetDefault<UStageMonitoringSettings>()->CommandLineFriendlyName;
		if (Descriptor.FriendlyName == NAME_None)
		{
			const FString TempName = FString::Printf(TEXT("%s:%d"), *Descriptor.MachineName, Descriptor.ProcessId);
			Descriptor.FriendlyName = *TempName;
		}

		Descriptor.SessionId = GetDefault<UStageMonitoringSettings>()->GetStageSessionId();

		return Descriptor;
	}
}


