// Copyright Epic Games, Inc. All Rights Reserved.

#include "StageMonitoringSettings.h"

#include "Misc/CommandLine.h"

UStageMonitoringSettings::UStageMonitoringSettings()
{
	int32 CommandLineSessionIdTmp = 0;
	if (FParse::Value(FCommandLine::Get(), TEXT("-StageSessionId="), CommandLineSessionIdTmp))
	{
		CommandLineSessionId = CommandLineSessionIdTmp;
	}

	FParse::Value(FCommandLine::Get(), TEXT("-StageFriendlyName="), CommandLineFriendlyName);
}

FName UStageMonitoringSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR
FText UStageMonitoringSettings::GetSectionText() const
{
	return NSLOCTEXT("StageMonitoringPlugin", "StageMonitorSettingsSection", "Stage Monitor");
}
#endif

int32 UStageMonitoringSettings::GetStageSessionId() const
{
	if (CommandLineSessionId.IsSet())
	{
		return CommandLineSessionId.GetValue();
	}

	return StageSessionId;
}

FStageMonitorSettings::FStageMonitorSettings()
{
	int32 bCommandLineAutoStartTemp = 0;
	if (FParse::Value(FCommandLine::Get(), TEXT("-StageMonitorAutoStart="), bCommandLineAutoStartTemp))
	{
		bCommandLineAutoStart = bCommandLineAutoStartTemp == 1;
	}
}

bool FStageMonitorSettings::ShouldAutoStartOnLaunch() const
{
	if (bCommandLineAutoStart.IsSet())
	{
		return bCommandLineAutoStart.GetValue();
	}

	return bAutoStart;
}

