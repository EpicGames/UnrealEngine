// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "StageMessages.h"

#include "StageMonitorUtils.generated.h"

UENUM()
enum class EStageMonitorNodeStatus
{
	Unknown,
	LoadingMap,
	Ready,
	HotReload,
	AssetCompiling
};


/**
 * Message containing information about frame timings.
 * Sent at regular intervals
 */
USTRUCT()
struct STAGEMONITORCOMMON_API FFramePerformanceProviderMessage : public FStageProviderPeriodicMessage
{
	GENERATED_BODY()

public:

	FFramePerformanceProviderMessage() = default;

	FFramePerformanceProviderMessage(EStageMonitorNodeStatus InStatus,
									 float GameThreadTime, float GameThreadWaitTime,
									 float RenderThreadTime, float RenderThreadWaitTime,
									 float GPUTime, float IdleTime, uint64 CPUMem, uint64 GPUMem, int32 InAssetsToCompile)
		: Status(InStatus)
		, GameThreadMS(GameThreadTime)
		, GameThreadWaitMS(GameThreadWaitTime)
		, RenderThreadMS(RenderThreadTime)
		, RenderThreadWaitMS(RenderThreadWaitTime)
		, GPU_MS(GPUTime)
		, IdleTimeMS(IdleTime)
		, CPU_MEM(CPUMem)
		, GPU_MEM(GPUMem)
		, CompilationTasksRemaining(InAssetsToCompile)
	{
		extern ENGINE_API float GAverageFPS;
		AverageFPS = GAverageFPS;
	}

	/** Average FrameRate read from GAverageFPS */
	UPROPERTY(VisibleAnywhere, Category = "Performance")
	EStageMonitorNodeStatus Status = EStageMonitorNodeStatus::Unknown;

	/** Full path name of the asset involved in the status. This will be empty for Ready, Shader Compiles, and Unknown states  */
	UPROPERTY(VisibleAnywhere, Category = "Performance")
	FString AssetInStatus;

	/** Average FrameRate read from GAverageFPS */
	UPROPERTY(VisibleAnywhere, Category = "Performance")
	float AverageFPS = 0.f;

	/** Current GameThread time read from GGameThreadTime in milliseconds */
	UPROPERTY(VisibleAnywhere, Category = "Performance", meta = (Unit = "ms"))
	float GameThreadMS = 0.f;

	/** Current GameThread wait time read from GGameThreadWaitTime in milliseconds */
	UPROPERTY(VisibleAnywhere, Category = "Performance", meta = (Unit = "ms"))
	float GameThreadWaitMS = 0.f;

	/** Current RenderThread time read from GRenderThreadTime in milliseconds */
	UPROPERTY(VisibleAnywhere, Category = "Performance", meta = (Unit = "ms"))
	float RenderThreadMS = 0.f;

	/** Current RenderThread wait time read from GRenderThreadWaitTime in milliseconds */
	UPROPERTY(VisibleAnywhere, Category = "Performance", meta = (Unit = "ms"))
	float RenderThreadWaitMS = 0.f;

	/** Current GPU time read from RHIGetGPUFrameCycles() in milliseconds */
	UPROPERTY(VisibleAnywhere, Category = "Performance", meta = (Unit = "ms"))
	float GPU_MS = 0.f;

	/** Idle time (slept) in milliseconds during last frame */
	UPROPERTY(VisibleAnywhere, Category = "Performance", meta = (Unit = "ms"))
	float IdleTimeMS = 0.f;

	/** Current CPU Memory Usage (Physical) in bytes */
	UPROPERTY(VisibleAnywhere, Category = "Performance")
	uint64 CPU_MEM = 0;

	/** Current GPU Memory Usage (Physical) in bytes */
	UPROPERTY(VisibleAnywhere, Category = "Performance")
	uint64 GPU_MEM = 0;

	/** Number of asynchronous compilation tasks currently in progress. */
	UPROPERTY(VisibleAnywhere, Category = "Performance")
	int32 CompilationTasksRemaining = 0;
};


namespace StageMonitorUtils
{
	STAGEMONITORCOMMON_API FStageInstanceDescriptor GetInstanceDescriptor();
}
