// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "GameplayTagContainer.h"
#include "Misc/FrameRate.h"
#include "StageMessages.h"

#include "StageMonitoringSettings.generated.h"

/**
 * Wrapper structure holding a message type static struct.
 * Used with a customization to generate a filtered list
 */
USTRUCT()
struct STAGEMONITORCOMMON_API FStageMessageTypeWrapper
{
	GENERATED_BODY()

public:
	FStageMessageTypeWrapper() = default;
	FStageMessageTypeWrapper(FName Type)
		: MessageType(Type)
	{}

	bool operator==(const FStageMessageTypeWrapper& Other) const { return MessageType == Other.MessageType; }
	bool operator==(const FName Other) const { return MessageType == Other; }

public:
	/** Name of StaticStruct message type */
	UPROPERTY(config, EditAnywhere, Category = "Stage Message")
	FName MessageType;

	friend uint32 GetTypeHash(const FStageMessageTypeWrapper& TypeWrapper)
	{
		return GetTypeHash(TypeWrapper.MessageType);
	}
};

/**
 * Settings associated to file exporter
 */
USTRUCT()
struct STAGEMONITORCOMMON_API FStageDataExportSettings
{
	GENERATED_BODY()

public:

	/** 
	 * Save only the last instance of periodic message types
	 * True by default to reduce file size
	 */
	UPROPERTY(config, EditAnywhere, Category = "Export")
	bool bKeepOnlyLastPeriodMessage = true;
	
	/** Message types to exclude from session when exporting */
	UPROPERTY(config, EditAnywhere, Category = "Export")
	TArray<FStageMessageTypeWrapper> ExcludedMessageTypes;
};

/**
 * Settings related to FramePerformance messages
 */
USTRUCT()
struct STAGEMONITORCOMMON_API FStageFramePerformanceSettings
{
	GENERATED_BODY()

public:

	/** Target FPS we're aiming for. */
	UPROPERTY(config, EditAnywhere, Category = "Frame Performance", meta = (Unit = "s"))
	float UpdateInterval = 0.2f;
};

/**
 * Settings related to HitchDetection messages
 */
USTRUCT()
struct STAGEMONITORCOMMON_API FStageHitchDetectionSettings
{
	GENERATED_BODY()

public:

	/** 
	 * Whether or not hitch detection should be enabled
	 * @note: This uses stat data. To avoid having on-screen message
	 * GAreScreenMessagesEnabled = false or -ExecCmds="DISABLEALLSCREENMESSAGES" on command line
	 * will turn them off.
	 * For more accurate hitch detection, use genlock which
	 * will have better missed frames information
	 */
	UPROPERTY(config, EditAnywhere, Category = "Hitch Detection")
	bool bEnableHitchDetection = false;

	/** Target FPS we're aiming for.  */
	UPROPERTY(config, EditAnywhere, Category = "Hitch Detection")
	FFrameRate MinimumFrameRate = FFrameRate(24, 1);
};

/**
 * Settings associated to DataProviders
 */
USTRUCT()
struct STAGEMONITORCOMMON_API FStageDataProviderSettings
{
	GENERATED_BODY()

public:

	/** If true, DataProvider will only start if machine has a role contained in SupportedRoles */
	UPROPERTY(config, EditAnywhere, Category = "Settings", meta = (InlineEditConditionToggle))
	bool bUseRoleFiltering = false;

	/** If checked, VP Role of this instance must be part of these roles to have the monitor operational */
	UPROPERTY(config, EditAnywhere, Category = "Settings", meta = (EditCondition = "bUseRoleFiltering"))
	FGameplayTagContainer SupportedRoles;

	UPROPERTY(config, EditAnywhere, Category = "Settings")
	TMap<FStageMessageTypeWrapper, FGameplayTagContainer> MessageTypeRoleExclusion;

	/** Settings about Frame Performance messaging */
	UPROPERTY(config, EditAnywhere, Category = "Settings")
	FStageFramePerformanceSettings FramePerformanceSettings;

	/** Settings about Hitch detection*/
	UPROPERTY(config, EditAnywhere, Category = "Settings")
	FStageHitchDetectionSettings HitchDetectionSettings;
};

/**
 * Settings for StageMonitor
 */
USTRUCT()
struct STAGEMONITORCOMMON_API FStageMonitorSettings
{
	GENERATED_BODY()

public:
	FStageMonitorSettings();

	/** Returns true if Monitor should start on launch. Can be overriden through commandline */
	bool ShouldAutoStartOnLaunch() const;

public:
	/** 
	 * If true, Monitor will only autostart if machine has a role contained in SupportedRoles 
	 * Once in editor, you can always start a monitor manually independently of the roles.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Settings", meta = (InlineEditConditionToggle))
	bool bUseRoleFiltering = false;

	/** If checked, VP Role of this instance must be part of these roles to have the monitor operational */
	UPROPERTY(config, EditAnywhere, Category = "Settings", meta = (EditCondition = "bUseRoleFiltering"))
	FGameplayTagContainer SupportedRoles;

	/** Interval between each discovery signal sent by Monitors */
	UPROPERTY(config, EditAnywhere, Category = "Settings", meta = (Units = "s"))
	float DiscoveryMessageInterval = 2.0f;

protected:
	/**
	 * Whether we should start monitoring on launch.
	 * @note It may be overriden via the command line, "-StageMonitorAutoStart=1 and via command line in editor"
	 */
	UPROPERTY(config, EditAnywhere, Category = "Settings")
	bool bAutoStart = false;

	/**
	 * Override AutoStart project settings through command line.
	 * ie. "-StageMonitorAutoStart=1"
	 */
	TOptional<bool> bCommandLineAutoStart;
};

/**
 * Settings for the StageMonitoring plugin modules. 
 * Data Provider, Monitor and shared settings are contained here to centralize access through project settings
 */
UCLASS(config=Game)
class STAGEMONITORCOMMON_API UStageMonitoringSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UStageMonitoringSettings();

	//~ Begin UDevelopperSettings interface
	virtual FName GetCategoryName() const;
#if WITH_EDITOR
	virtual FText GetSectionText() const override;
#endif
	//~ End UDevelopperSettings interface


	/** Returns current SessionId either based on settings or overriden by commandline */
	int32 GetStageSessionId() const;

public:

	/** If true, Stage monitor will only listen to Stage Providers with same sessionId */
	UPROPERTY(config, EditAnywhere, Category = "Settings")
	bool bUseSessionId = true;

protected:
	/**
	 * The projects Stage SessionId to differentiate data sent over network.
	 * @note It may be overriden via the command line, "-StageSessionId=1"
	 */
	UPROPERTY(config, EditAnywhere, Category = "Settings")
	int32 StageSessionId = INDEX_NONE;

public:

	/** Interval threshold between message reception before dropping out a provider or a monitor */
	UPROPERTY(config, EditAnywhere, Category = "Settings", meta=(Unit="s"))
	float TimeoutInterval = 10.0f;

	/** Settings for monitors */
	UPROPERTY(config, EditAnywhere, Category = "Monitor Settings")
	FStageMonitorSettings MonitorSettings;

	/** Settings for Data Providers */
	UPROPERTY(config, EditAnywhere, Category = "Provider Settings")
	FStageDataProviderSettings ProviderSettings;

	/** Settings for Data Providers */
	UPROPERTY(config, EditAnywhere, Category = "Export Settings")
	FStageDataExportSettings ExportSettings;


	/**
	 * The current SessionId in a virtual production context read from the command line.
	 * ie. "-StageSessionId=1"
	 */
	TOptional<int32> CommandLineSessionId;

	/**
	 * A friendly name for that instance given through command line (-StageFriendlyName=) to identify it when monitoring.
	 * If none, by default it will be filled with MachineName:ProcessId
	 */
	FName CommandLineFriendlyName;
};