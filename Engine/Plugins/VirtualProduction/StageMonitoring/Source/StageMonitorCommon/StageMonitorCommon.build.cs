// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class StageMonitorCommon : ModuleRules
{
	public StageMonitorCommon(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"DeveloperSettings",
				"StageDataCore"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"GameplayTags",
				"VPRoles",
				"VPUtilities",
			}
		);
	}
}
