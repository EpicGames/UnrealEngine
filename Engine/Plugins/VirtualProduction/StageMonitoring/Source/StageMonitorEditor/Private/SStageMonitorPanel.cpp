// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStageMonitorPanel.h"

#include "DesktopPlatformModule.h"
#include "EditorFontGlyphs.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDesktopPlatform.h"
#include "ISettingsModule.h"
#include "IStageMonitor.h"
#include "IStageMonitorSession.h"
#include "IStageMonitorSessionManager.h"
#include "IStageMonitorModule.h"
#include "LevelEditor.h"
#include "Misc/Paths.h"
#include "SlateOptMacros.h"
#include "StageMonitorEditorStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SDataProviderActivities.h"
#include "Widgets/SDataProviderListView.h"

#define LOCTEXT_NAMESPACE "SStageMonitorPanel"


TWeakPtr<SStageMonitorPanel> SStageMonitorPanel::PanelInstance;
FDelegateHandle SStageMonitorPanel::LevelEditorTabManagerChangedHandle;


namespace StageMonitorUtilities
{
	static const FName NAME_App = FName("StageMonitorPanelApp");
	static const FName NAME_MessageViewerName = FName("StageMessageViewer");
	static const FName NAME_LevelEditorModuleName = FName("LevelEditor");

	TSharedRef<SDockTab> CreateApp(const FSpawnTabArgs& Args)
	{
		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SNew(SStageMonitorPanel)
			];
	}
}


void SStageMonitorPanel::RegisterNomadTabSpawner(TSharedRef<FWorkspaceItem> InWorkspaceItem)
{
	auto RegisterTabSpawner = [InWorkspaceItem]()
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(StageMonitorUtilities::NAME_LevelEditorModuleName);
		TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

		LevelEditorTabManager->RegisterTabSpawner(StageMonitorUtilities::NAME_App, FOnSpawnTab::CreateStatic(&StageMonitorUtilities::CreateApp))
			.SetDisplayName(LOCTEXT("TabTitle", "Stage Monitor"))
			.SetTooltipText(LOCTEXT("TooltipText", "Monitor performance data from stage machines"))
			.SetGroup(InWorkspaceItem)
			.SetIcon(FSlateIcon(FStageMonitorEditorStyle::Get().GetStyleSetName(), "StageMonitor.TabIcon"));
	};

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(StageMonitorUtilities::NAME_LevelEditorModuleName);
	if (LevelEditorModule.GetLevelEditorTabManager())
	{
		RegisterTabSpawner();
	}
	else
	{
		LevelEditorTabManagerChangedHandle = LevelEditorModule.OnTabManagerChanged().AddLambda(RegisterTabSpawner);
	}
}

void SStageMonitorPanel::UnregisterNomadTabSpawner()
{
	if (FSlateApplication::IsInitialized() && FModuleManager::Get().IsModuleLoaded(StageMonitorUtilities::NAME_LevelEditorModuleName))
	{
		FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(StageMonitorUtilities::NAME_LevelEditorModuleName);
		TSharedPtr<FTabManager> LevelEditorTabManager;
		if (LevelEditorModule)
		{
			LevelEditorTabManager = LevelEditorModule->GetLevelEditorTabManager();
			LevelEditorModule->OnTabManagerChanged().Remove(LevelEditorTabManagerChangedHandle);
		}

		if (LevelEditorTabManager.IsValid())
		{
			LevelEditorTabManager->UnregisterTabSpawner(StageMonitorUtilities::NAME_App);
		}
	}
}

TSharedPtr<SStageMonitorPanel> SStageMonitorPanel::GetPanelInstance()
{
	return SStageMonitorPanel::PanelInstance.Pin();
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SStageMonitorPanel::Construct(const FArguments& InArgs)
{
	PanelInstance = StaticCastSharedRef<SStageMonitorPanel>(AsShared());

	//Register to session being loaded/saved to update UI
	IStageMonitorModule::Get().GetStageMonitorSessionManager().OnStageMonitorSessionLoaded().AddSP(this, &SStageMonitorPanel::OnStageMonitorSessionLoaded);
	IStageMonitorModule::Get().GetStageMonitorSessionManager().OnStageMonitorSessionSaved().AddSP(this, &SStageMonitorPanel::OnStageMonitorSessionSaved);

	ChildSlot
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SVerticalBox)
			// Toolbar
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.f, 5.f, 5.f, 5.f)
			[
				MakeToolbarWidget()
			]
			+ SVerticalBox::Slot()
			.Padding(5.f, 5.f, 5.f, 5.f)
			.FillHeight(.8)
			[
				SNew(SSplitter)
				.Orientation(Orient_Vertical)
				+ SSplitter::Slot()
				.Value(.25f)
				[
					// Data Provider List
					SNew(SBorder)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
					[
						SAssignNew(DataProviderList, SDataProviderListView, CurrentSession)
					]
				]
				+ SSplitter::Slot()
				.Value(.75f)
				[
					// Data Provider Activities
					SNew(SBorder)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
					[
						SAssignNew(DataProviderActivities, SDataProviderActivities, SharedThis<SStageMonitorPanel>(this), CurrentSession)
					]
				]
			]
			// Monitor status
			+ SVerticalBox::Slot()
			.Padding(5.f, 5.f, 5.f, 5.f)
			.AutoHeight()
			[
				SNew(SBorder)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
				[
					SNew(SHorizontalBox)
					//Monitor session info
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.Padding(10.f, 10.f, 10.f, 10.f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("MonitorSession", "CurrentSession : "))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.Padding(10.f, 10.f, 10.f, 10.f)
					[
						SNew(STextBlock)
						.Text(this, &SStageMonitorPanel::GetCurrentSessionInfo)
					]
					// Spacer
					+ SHorizontalBox::Slot()
					[
						SNew(SSpacer)
					]
					// Monitor status
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.Padding(10.f, 10.f, 10.f, 10.f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("MonitorStatus", "Monitor Status : "))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.Padding(10.f, 10.f, 10.f, 10.f)
					[
						SNew(SCheckBox)
						.Style(FStageMonitorEditorStyle::Get(), "ViewMode")
						.ToolTipText(LOCTEXT("ActivateMonitorTooltip", "Start / stop monitor"))
						.IsChecked(this, &SStageMonitorPanel::IsMonitorActive)
						.OnCheckStateChanged(this, &SStageMonitorPanel::OnMonitorStateChanged)
						[
							SNew(STextBlock)
							.Text(this, &SStageMonitorPanel::GetMonitorStatus)
						]
					]
				]
			]
		]
		+ SOverlay::Slot()
		[
			//Overlay present when saving/loading requests are being done
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("SettingsEditor.CheckoutWarningBorder"))
			.BorderBackgroundColor(FLinearColor(0.2f, 0.2f, 0.2f, 0.5f))
			.Padding(FMargin(5.f, 5.f, 5.f, 5.f))
			.Visibility(this, &SStageMonitorPanel::GetThrobberVisibility)
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Clipping(EWidgetClipping::ClipToBounds)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					[
						SNew(SCircularThrobber)
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					[
						SNew(SButton)
						.ContentPadding(FMargin(4.f, 2.f))
						.OnClicked(this, &SStageMonitorPanel::OnCancelRequest)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("CancelLabel", "Cancel"))
						]
					]
				]
			]
		]
		
	];

	RefreshDisplayedSession();
}

TSharedRef<SWidget> SStageMonitorPanel::MakeToolbarWidget()
{
	TSharedRef<SWidget> Toolbar = SNew(SBorder)
	.VAlign(VAlign_Center)
	.Padding(FMargin(5.f, 5.f, 5.f, 5.f))
	[

		SNew(SHorizontalBox)
		//Live/Preview mode
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4.f)
		[
			SNew(SCheckBox)
			.Style(FStageMonitorEditorStyle::Get(), "ViewMode")
			.ToolTipText(LOCTEXT("ViewModeTooltip", "Toggles view mode between live session and loaded session"))
			.IsChecked(this, &SStageMonitorPanel::GetViewMode)
			.OnCheckStateChanged(this, &SStageMonitorPanel::OnViewModeChanged)
		]
		// Load previous session
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4.f)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "ToggleButton")
			.ToolTipText(LOCTEXT("LoadSession", "Loads a previous stage monitoring session."))
			.ContentPadding(FMargin(4.f, 4.f))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SStageMonitorPanel::OnLoadSessionClicked)
			.IsEnabled(MakeAttributeLambda([this] { return !bIsShowingLiveSession; }))
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
				.Text(FEditorFontGlyphs::Folder_Open)
				.ColorAndOpacity(FLinearColor::White)
			]
		]
		// Save current session
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4.f)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "ToggleButton")
			.ToolTipText(LOCTEXT("SaveSession", "Save current stage monitoring session."))
			.ContentPadding(FMargin(4.f, 4.f))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SStageMonitorPanel::OnSaveSessionClicked)
			.IsEnabled(MakeAttributeLambda([this] { return bIsShowingLiveSession; }))
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
				.Text(FText::FromString(FString(TEXT("\xf0c7"))) /*fa-save*/)
				.ColorAndOpacity(FLinearColor::White)
			]
		]
		// Clear entries buttons
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "ToggleButton")
				.ToolTipText(LOCTEXT("ClearButton_ToolTip", "Clear unresponsive providers and all entries."))
				.ContentPadding(FMargin(4.f, 4.f))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked(this, &SStageMonitorPanel::OnClearClicked)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
					.Text(FEditorFontGlyphs::Eraser)
					.ColorAndOpacity(FLinearColor::White)
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SComboButton)
				.ButtonStyle(FAppStyle::Get(), "ToggleButton")
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnGetMenuContent(this, &SStageMonitorPanel::OnClearBuildMenu)
			]
		]
		// todo : Show Message Viewer 
		/*+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4.f)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "ToggleButton")
			.ToolTipText(LOCTEXT("ShowMessageViewer_ToolTip", "Open the message viewer"))
			.ContentPadding(FMargin(4, 2))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SStageMonitorPanel::OnShowMessageViewerClicked)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("GenericViewButton"))
			]
		]*/
		// Settings button
		+ SHorizontalBox::Slot()
		.Padding(4.f)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "ToggleButton")
			.ToolTipText(LOCTEXT("ShowProjectSettings_Tip", "Show the StageMonitor project settings"))
			.ContentPadding(FMargin(4.f, 4.f))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SStageMonitorPanel::OnShowProjectSettingsClicked)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
				.Text(FEditorFontGlyphs::Cogs)
				.ColorAndOpacity(FLinearColor::White)
			]
		]
		// Spacer
		+ SHorizontalBox::Slot()
		[
			SNew(SSpacer)
		]
		// Stage status
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(4.f)
		[
			// Stage status icon
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.f)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.18"))
				.Text(FEditorFontGlyphs::Circle)
				.ColorAndOpacity(this, &SStageMonitorPanel::GetStageStatus)
			]
			// Stage status text
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.f)
			[
				SNew(SVerticalBox)
				// Critical state text
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("CriticalStateHeader", "Critical Source Name"))
					.ColorAndOpacity(FLinearColor::White)
				]
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &SStageMonitorPanel::GetStageActiveStateReasonText)
					.TextStyle(FAppStyle::Get(), "LargeText")
					.ColorAndOpacity(FLinearColor::White)
				]
			]
		]
	];

	return Toolbar;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FReply SStageMonitorPanel::OnClearClicked()
{
	OnClearUnresponsiveProvidersClicked();
	OnClearEntriesClicked();

	return FReply::Handled();
}

void SStageMonitorPanel::OnClearEntriesClicked()
{
	if (CurrentSession.IsValid())
	{
		CurrentSession->ClearAll();
	}
}

void SStageMonitorPanel::OnClearUnresponsiveProvidersClicked()
{
	if (CurrentSession.IsValid())
	{
		CurrentSession->ClearUnresponsiveProviders();
	}
}

TSharedRef<SWidget> SStageMonitorPanel::OnClearBuildMenu()
{
	constexpr bool bShouldCloseMenuAfterSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseMenuAfterSelection, nullptr);

	//Reset the unresponsive sources and entries from the session
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ClearUnresponsiveProvidersLabel", "Clear Unresponsive Providers"),
		LOCTEXT("ClearUnresponsiveProvidersTooltip", "Clears providers that not communicating anymore."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SStageMonitorPanel::OnClearUnresponsiveProvidersClicked),
			FCanExecuteAction(),
			FIsActionChecked())
	);

	//Reset all entries currently logged for this session
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ClearEntriesLabel", "Clear All Entries"),
		LOCTEXT("ClearEntriesTooltip", "Clears all entries received during the session."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SStageMonitorPanel::OnClearEntriesClicked),
			FCanExecuteAction(),
			FIsActionChecked())
	);

	return MenuBuilder.MakeWidget();
}

FReply SStageMonitorPanel::OnShowProjectSettingsClicked()
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Project", "Plugins", "StageMonitoringSettings");
	return FReply::Handled();
}

FReply SStageMonitorPanel::OnLoadSessionClicked()
{
	if (bIsShowingLiveSession == false && bIsWaitingForAsyncResult == false)
	{
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);
		const FString FileTypes = "StageSession|*.json";
		const FString Title = "Import previously saved stage monitoring session";
		const FString DefaultPath;
		const FString DefaultFile = TEXT("StageData.json");

		TArray<FString> OutFilenames;
		const bool bFileSelected = DesktopPlatform->OpenFileDialog(
			ParentWindowWindowHandle
			, Title
			, DefaultPath
			, DefaultFile
			, FileTypes
			, EFileDialogFlags::None
			, OutFilenames
		);

		if (bFileSelected && OutFilenames.Num() > 0)
		{
			const FString FullPath = FPaths::ConvertRelativePathToFull(OutFilenames[0]);
			bIsWaitingForAsyncResult = IStageMonitorModule::Get().GetStageMonitorSessionManager().LoadSession(FullPath);
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SStageMonitorPanel::OnSaveSessionClicked()
{
	if (bIsShowingLiveSession == true && bIsWaitingForAsyncResult == false)
	{
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);
		const FString FileTypes = "StageSession|*.json";
		const FString Title = "Save current stage monitoring data";
		const FString DefaultPath;
		const FString DefaultFile = TEXT("StageData.json");

		TArray<FString> OutFilenames;
		const bool bFileSelected = DesktopPlatform->SaveFileDialog(
			ParentWindowWindowHandle
			, Title
			, DefaultPath
			, DefaultFile
			, FileTypes
			, EFileDialogFlags::None
			, OutFilenames
		);

		if (bFileSelected && OutFilenames.Num() > 0)
		{
			const FString FullPath = FPaths::ConvertRelativePathToFull(OutFilenames[0]);
			bIsWaitingForAsyncResult = IStageMonitorModule::Get().GetStageMonitorSessionManager().SaveSession(FullPath);
		}

		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

FSlateColor SStageMonitorPanel::GetStageStatus() const
{
	if (CurrentSession.IsValid() && CurrentSession->IsStageInCriticalState())
	{
		return FLinearColor::Red;
	}

	return FLinearColor::Green;
}

FText SStageMonitorPanel::GetStageActiveStateReasonText() const
{
	if(CurrentSession.IsValid())
	{
		return FText::FromName(CurrentSession->GetCurrentCriticalStateSource());
	}

	return FText::GetEmpty();
}

FText SStageMonitorPanel::GetMonitorStatus() const
{
	IStageMonitor& StageMonitor = IStageMonitorModule::Get().GetStageMonitor();
	if (StageMonitor.IsActive())
	{
		return LOCTEXT("MonitorStatusActive", "Active");
	}
	else
	{
		return LOCTEXT("MonitorStatusInactive", "Inactive");
	}
}

ECheckBoxState SStageMonitorPanel::IsMonitorActive() const
{
	IStageMonitor& StageMonitor = IStageMonitorModule::Get().GetStageMonitor();
	return StageMonitor.IsActive() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SStageMonitorPanel::OnMonitorStateChanged(ECheckBoxState NewState)
{
	IStageMonitorModule::Get().EnableMonitor(NewState == ECheckBoxState::Checked);
}

FText SStageMonitorPanel::GetCurrentSessionInfo() const
{
	if (CurrentSession)
	{
		return FText::FromString(CurrentSession->GetSessionName());
	}
	else
	{
		return LOCTEXT("SessionInfoLive", "None");
	}
}

void SStageMonitorPanel::OnStageMonitorSessionLoaded()
{
	if (bIsWaitingForAsyncResult && bIsShowingLiveSession == false)
	{
		RefreshDisplayedSession();
		bIsWaitingForAsyncResult = false;
	}
}

void SStageMonitorPanel::OnStageMonitorSessionSaved()
{
	if (bIsWaitingForAsyncResult && bIsShowingLiveSession == true)
	{
		bIsWaitingForAsyncResult = false;
	}
}

ECheckBoxState SStageMonitorPanel::GetViewMode() const
{
	if (bIsShowingLiveSession)
	{
		return ECheckBoxState::Checked;
	}
	else
	{
		return ECheckBoxState::Unchecked;
	}
}

void SStageMonitorPanel::OnViewModeChanged(ECheckBoxState NewState)
{
	if (NewState == ECheckBoxState::Checked)
	{
		bIsShowingLiveSession = true;

	}
	else
	{
		bIsShowingLiveSession = false;
	}

	RefreshDisplayedSession();
}

void SStageMonitorPanel::RefreshDisplayedSession()
{
	TSharedPtr<IStageMonitorSession> PreviousSession = CurrentSession;
	if (bIsShowingLiveSession)
	{
		CurrentSession = IStageMonitorModule::Get().GetStageMonitorSessionManager().GetActiveSession();
	}
	else
	{
		CurrentSession = IStageMonitorModule::Get().GetStageMonitorSessionManager().GetLoadedSession();
	}

	if (CurrentSession != PreviousSession)
	{
		DataProviderList->RefreshMonitorSession(CurrentSession);
		DataProviderActivities->RefreshMonitorSession(CurrentSession);
	}
}

EVisibility SStageMonitorPanel::GetThrobberVisibility() const
{
	return bIsWaitingForAsyncResult ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SStageMonitorPanel::OnCancelRequest()
{
	//Get rid of the throbber if a user wants out. Will need to do an action on the session manager at some point
	if (bIsWaitingForAsyncResult)
	{
		bIsWaitingForAsyncResult = false;
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
