// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"


enum class ECheckBoxState : uint8;


class IStageMonitorSession;
class SDataProviderListView;
class SDataProviderActivities;
class FWorkspaceItem;

/**
 * Panel used to show stage monitoring data
 */
class SStageMonitorPanel : public SCompoundWidget
{
public:
	virtual ~SStageMonitorPanel() = default;

	static void RegisterNomadTabSpawner(TSharedRef<FWorkspaceItem> InWorkspaceItem);
	static void UnregisterNomadTabSpawner();
	static TSharedPtr<SStageMonitorPanel> GetPanelInstance();

private:
	using Super = SCompoundWidget;
	static TWeakPtr<SStageMonitorPanel> PanelInstance;
	static FDelegateHandle LevelEditorTabManagerChangedHandle;

public:
	SLATE_BEGIN_ARGS(SStageMonitorPanel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	/** Make the toolbar widgets */
	TSharedRef<SWidget> MakeToolbarWidget();

	/** Handles clear everything button clicked */
	FReply OnClearClicked();
	
	/** Clears all entries from the current session */
	void OnClearEntriesClicked();

	/** Clears all unresponsive providers */
	void OnClearUnresponsiveProvidersClicked();

	/** Builds a menu for specific clearing option */
	TSharedRef<SWidget> OnClearBuildMenu();

	/** Handles StageMonitor settings button clicked */
	FReply OnShowProjectSettingsClicked();

	/** Handles load session button clicked */
	FReply OnLoadSessionClicked();

	/** Handles save session button clicked */
	FReply OnSaveSessionClicked();

	/** Get the stage status */
	FSlateColor GetStageStatus() const;

	/** Get the top reason why the stage is considered active */
	FText GetStageActiveStateReasonText() const;

	/** Returns the monitor status whether it's actively listening for data providers or not */
	FText GetMonitorStatus() const;

	/** Returns whether the monitor is active or not for checkbox state */
	ECheckBoxState IsMonitorActive() const;

	/** Callback when user toggles the monitor state switch */
	void OnMonitorStateChanged(ECheckBoxState NewState);

	/** Returns information about the displayed session */
	FText GetCurrentSessionInfo() const;

	/** Callback when a requested session was loaded */
	void OnStageMonitorSessionLoaded();

	/** Callback when a save session request was completed */
	void OnStageMonitorSessionSaved();

	/** Returns whether we're displaying live session or loaded one*/
	ECheckBoxState GetViewMode() const;

	/** Called when view mode switch is clicked */
	void OnViewModeChanged(ECheckBoxState NewState);

	/** Updates UI when a new session was loaded or live mode is activated */
	void RefreshDisplayedSession();

	/** When waiting for loading/saving session display the throbber */
	EVisibility GetThrobberVisibility() const;

	/** Used to cancel loading or saving that is in progress */
	FReply OnCancelRequest();

private:

	/** Used to show all providers with their frame data */
	TSharedPtr<SDataProviderListView> DataProviderList;

	/** Used to show every activities received by the monitor */
	TSharedPtr<SDataProviderActivities> DataProviderActivities;

	/** Session shown by the monitor editor */
	TSharedPtr<IStageMonitorSession> CurrentSession;

	/** Driven by the toggle switch from the UI. Keeps wheter we are showing live data or imported */
	bool bIsShowingLiveSession = true;

	/** When we request saving or loading, this will be true to show update UI */
	bool bIsWaitingForAsyncResult = false;
};
