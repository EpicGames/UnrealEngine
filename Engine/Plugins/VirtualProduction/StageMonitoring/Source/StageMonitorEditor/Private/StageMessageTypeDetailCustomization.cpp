// Copyright Epic Games, Inc. All Rights Reserved.

#include "StageMessageTypeDetailCustomization.h"

#include "DetailWidgetRow.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IPropertyUtilities.h"
#include "StageMessages.h"
#include "StageMonitoringSettings.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Input/SComboButton.h"


#define LOCTEXT_NAMESPACE "StageMessageTypeDetailCustomization"


TSharedRef<IPropertyTypeCustomization> FStageMessageTypeDetailCustomization::MakeInstance()
{
	return MakeShareable(new FStageMessageTypeDetailCustomization);
}

void FStageMessageTypeDetailCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	StructPropertyHandle = InPropertyHandle;
	TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();

	check(CastFieldChecked<FStructProperty>(StructPropertyHandle->GetProperty())->Struct == FStageMessageTypeWrapper::StaticStruct());


	TArray<void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);

	check(RawData.Num() == 1);
	FStageMessageTypeWrapper* MessageTypeWrapper = reinterpret_cast<FStageMessageTypeWrapper*>(RawData[0]);

	//Write Name content as is
	HeaderRow.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MaxDesiredWidth(512)
	[
		//Customize value to filter for our specific UStruct base type
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			//Show current selection name
			SNew(STextBlock)
			.Text(MakeAttributeLambda([=] { return FText::FromName(MessageTypeWrapper->MessageType); }))
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		.VAlign(VAlign_Center)
		[
			//Add combo button to select from a drop down possible values
			SNew(SComboButton)
			.OnGetMenuContent(this, &FStageMessageTypeDetailCustomization::HandleSourceComboButtonMenuContent)
			.ContentPadding(FMargin(4.0, 2.0))
		]
	].IsEnabled(MakeAttributeLambda([=] { return !InPropertyHandle->IsEditConst() && PropertyUtils->IsPropertyEditingEnabled(); }));
}


TSharedRef<SWidget> FStageMessageTypeDetailCustomization::HandleSourceComboButtonMenuContent()
{
	// Generate menu
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection("AllStageMessageTypes", LOCTEXT("AllStageMessageTypesSection", "Stage Message Types"));
	{
		for (TObjectIterator<UScriptStruct> It; It; ++It)
		{
			UScriptStruct* Struct = *It;
			const UScriptStruct* BasePeriodicMessage = FStageProviderPeriodicMessage::StaticStruct();
			const UScriptStruct* BaseEventMessage = FStageProviderEventMessage::StaticStruct();
			const bool bIsValidMessageStruct = (BasePeriodicMessage && Struct->IsChildOf(BasePeriodicMessage) && (Struct != BasePeriodicMessage))
				|| (BaseEventMessage && Struct->IsChildOf(BaseEventMessage) && (Struct != BaseEventMessage));

			if (bIsValidMessageStruct)
			{
				MenuBuilder.AddMenuEntry
				(
					Struct->GetDisplayNameText(), //Label
					Struct->GetDisplayNameText(), //Tooltip
					FSlateIcon(),
					FUIAction
					(
						FExecuteAction::CreateLambda([this, Struct]
							{
								if (StructPropertyHandle.IsValid())
								{
									FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(StructPropertyHandle->GetProperty());

									TArray<void*> RawData;
									StructPropertyHandle->AccessRawData(RawData);
									if (RawData.Num() == 1 && RawData[0] != nullptr)
									{
										FStageMessageTypeWrapper* PreviousValue = reinterpret_cast<FStageMessageTypeWrapper*>(RawData[0]);
										FStageMessageTypeWrapper NewValue;
										NewValue.MessageType = Struct->GetFName();

										//Write out selection in the property
										FString TextValue;
										StructProperty->Struct->ExportText(TextValue, &NewValue, PreviousValue, nullptr, EPropertyPortFlags::PPF_None, nullptr);
										ensure(StructPropertyHandle->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
									}
								}
							})
						, FCanExecuteAction()
						, FIsActionChecked::CreateLambda([this, Struct]
							{
								if (StructPropertyHandle.IsValid())
								{
									FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(StructPropertyHandle->GetProperty());

									TArray<void*> RawData;
									StructPropertyHandle->AccessRawData(RawData);
									if (RawData.Num() == 1 && RawData[0] != nullptr)
									{
										FStageMessageTypeWrapper* CurrentValue = reinterpret_cast<FStageMessageTypeWrapper*>(RawData[0]);
										return CurrentValue->MessageType == Struct->GetFName();
									}
								}

								return false;
							})
					),
					NAME_None,
					EUserInterfaceActionType::RadioButton
				);
			}
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
