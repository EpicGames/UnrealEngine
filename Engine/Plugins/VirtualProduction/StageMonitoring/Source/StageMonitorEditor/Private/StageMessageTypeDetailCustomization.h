// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class SWidget;

class FStageMessageTypeDetailCustomization : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it. */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

public:
	// IDetailCustomization interface
	virtual void CustomizeHeader( TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils ) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override {}

private:
	TSharedRef<SWidget> HandleSourceComboButtonMenuContent();

private:

	/** Handle to the structure property we're editing */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;
};
