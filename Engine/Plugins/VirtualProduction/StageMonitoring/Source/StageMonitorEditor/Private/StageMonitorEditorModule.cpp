// Copyright Epic Games, Inc. All Rights Reserved.

#include "StageMonitorEditorModule.h"

#include "ISettingsModule.h"
#include "PropertyEditorModule.h"
#include "StageMessageTypeDetailCustomization.h"
#include "StageMonitorEditorSettings.h"
#include "StageMonitoringSettings.h"
#include "SStageMonitorPanel.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

DEFINE_LOG_CATEGORY(LogStageMonitorEditor);

#define LOCTEXT_NAMESPACE "StageMonitorEditor"


void FStageMonitorEditorModule::StartupModule()
{
	SStageMonitorPanel::RegisterNomadTabSpawner(WorkspaceMenu::GetMenuStructure().GetLevelEditorVirtualProductionCategory());

	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule)
	{
		SettingsModule->RegisterSettings("Project", "Plugins", "Stage Monitor Editor",
			LOCTEXT("StageMonitorEditorName", "Stage Monitor Editor"),
			LOCTEXT("StageMonitorEditorDescription", "Configure the editor aspects of StageMonitor plugin."),
			GetMutableDefault<UStageMonitorEditorSettings>());
	}
	

	RegisterCustomizations();
}

void FStageMonitorEditorModule::ShutdownModule()
{
	if (!IsRunningCommandlet() && UObjectInitialized() && !IsEngineExitRequested())
	{
		UnregisterCustomizations();

		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "Stage Monitor Editor");
		}

		SStageMonitorPanel::UnregisterNomadTabSpawner();
	}
}

void FStageMonitorEditorModule::RegisterCustomizations()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(FStageMessageTypeWrapper::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FStageMessageTypeDetailCustomization::MakeInstance));
}

void FStageMonitorEditorModule::UnregisterCustomizations()
{
	FPropertyEditorModule* PropertyEditorModule = FModuleManager::Get().GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyEditorModule)
	{
		PropertyEditorModule->UnregisterCustomPropertyTypeLayout(FStageMessageTypeWrapper::StaticStruct()->GetFName());
	}
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FStageMonitorEditorModule, StageMonitorEditor)
