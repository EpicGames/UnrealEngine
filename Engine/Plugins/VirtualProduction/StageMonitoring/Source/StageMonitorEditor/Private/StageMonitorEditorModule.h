// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"

DECLARE_LOG_CATEGORY_EXTERN(LogStageMonitorEditor, Log, All);

class FStageMonitorEditorModule : public IModuleInterface
{
public:	

	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~End IModuleInterface

private:
	/** Registers detail customization for stage monitoring types */
	void RegisterCustomizations();
	
	/** Unregisters registered customizations */
	void UnregisterCustomizations();
};
