// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "StageMonitorEditorSettings.generated.h"


/**
 * Settings for the editor aspect of the StageMonitoring plugin modules. 
 */
UCLASS(config = EditorPerProjectUserSettings, MinimalAPI)
class UStageMonitorEditorSettings : public UObject
{
	GENERATED_BODY()

public:

	/** Refresh rate in seconds for the StageMonitor panel. */
	UPROPERTY(Config, EditAnywhere, Category = "UI", meta = (ClampMin = 0.0f))
	float RefreshRate = 0.2f;

	/** Stores the state of the visibility settings in JSON format. */
	UPROPERTY(Config)
	FString ColumnVisibilitySettings;
};
