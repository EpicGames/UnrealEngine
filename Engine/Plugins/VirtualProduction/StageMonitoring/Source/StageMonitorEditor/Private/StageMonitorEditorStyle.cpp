// Copyright Epic Games, Inc. All Rights Reserved.

#include "StageMonitorEditorStyle.h"

#include "Brushes/SlateColorBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/CoreStyle.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"



FStageMonitorEditorStyle::FStageMonitorEditorStyle()
	: FSlateStyleSet("StageMonitorEditorStyle")
{
	const FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("StageMonitoring"))->GetContentDir();
	SetContentRoot(ContentDir);
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// Icons
	{
		const FVector2D Icon20x20(20.0f, 20.0f);
		Set("StageMonitor.TabIcon", new IMAGE_BRUSH_SVG("StageMonitor", Icon20x20));
	}

	// ListView
	{
		const FVector2D Icon8x8(8.0f, 8.0f);
		FTableRowStyle CriticalStateRow(FCoreStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row"));
		CriticalStateRow.SetEvenRowBackgroundBrush(FSlateColorBrush(FLinearColor(0.2f, 0.0f, 0.0f, 0.1f)))
			.SetOddRowBackgroundBrush(FSlateColorBrush(FLinearColor(0.2f, 0.0f, 0.0f, 0.1f)));

		Set("TableView.CriticalStateRow", CriticalStateRow);
	}

	//Checkbox for live/preview mode
	{
		FCheckBoxStyle SwitchStyle = FCheckBoxStyle()
		.SetForegroundColor(FLinearColor::White)
		.SetUncheckedImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_OFF.png")), FVector2D(28.0F, 14.0F)))
		.SetUncheckedHoveredImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_OFF.png")), FVector2D(28.0F, 14.0F)))
		.SetUncheckedPressedImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_OFF.png")), FVector2D(28.0F, 14.0F)))
		.SetCheckedImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_ON.png")), FVector2D(28.0F, 14.0F)))
		.SetCheckedHoveredImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_ON.png")), FVector2D(28.0F, 14.0F)))
		.SetCheckedPressedImage(FSlateImageBrush(RootToContentDir(TEXT("Switch_ON.png")), FVector2D(28.0F, 14.0F)))
		.SetPadding(FMargin(0, 0, 0, 1));

		Set("ViewMode", SwitchStyle);

	}

	
	FSlateStyleRegistry::RegisterSlateStyle(*this);

#undef IMAGE_BRUSH
}

FStageMonitorEditorStyle::~FStageMonitorEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FStageMonitorEditorStyle& FStageMonitorEditorStyle::Get()
{
	static FStageMonitorEditorStyle Inst;
	return Inst;
}

