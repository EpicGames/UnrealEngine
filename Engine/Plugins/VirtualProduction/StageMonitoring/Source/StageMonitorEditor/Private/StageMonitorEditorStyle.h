// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

/**
 * Slate style set that defines all the styles for the StageMonitor UI
 */
class STAGEMONITOREDITOR_API FStageMonitorEditorStyle : public FSlateStyleSet
{
public:
	/** Access the singleton instance for this style set */
	static FStageMonitorEditorStyle& Get();


private:
	FStageMonitorEditorStyle();
	~FStageMonitorEditorStyle();
};
