// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDataProviderActivities.h"

#include "DetailsViewArgs.h"
#include "IDetailsView.h"
#include "Framework/Views/TableViewMetadata.h"
#include "IStructureDetailsView.h"
#include "IStageMonitorSession.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SDataProviderActivityFilter.h"
#include "StageMonitorEditorStyle.h"
#include "Templates/Greater.h"
#include "UObject/StructOnScope.h"
#include "Widgets/Views/SListView.h"


static TAutoConsoleVariable<int32> CVarStageMonitorMinFramesBetweenRefresh(TEXT("StageMonitor.MinFramesBetweenRefresh"), 15, TEXT("The minimum number of frames between a UI refresh."));

#define LOCTEXT_NAMESPACE "SDataProviderActivities"

namespace DataProviderActivitiesListView
{
	const FName HeaderIdName_Timecode = "Timecode";
	const FName HeaderIdName_StageName = "StageName";
	const FName HeaderIdName_Type = "Type";
	const FName HeaderIdName_Description = "Description";
}

/**
 * SDataProviderActivities
 */
void SDataProviderActivities::Construct(const FArguments& InArgs, TSharedPtr<SStageMonitorPanel> InOwnerPanel, const TWeakPtr<IStageMonitorSession>& InSession)
{
	OwnerPanel = InOwnerPanel;

	AttachToMonitorSession(InSession);

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = true;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.ViewIdentifier = NAME_None;
	DetailsViewArgs.bShowCustomFilterOption = false;
	DetailsViewArgs.bShowOptions = false;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FStructureDetailsViewArgs StructViewArgs;
	StructureDetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructViewArgs, TSharedPtr<FStructOnScope>());
	StructureDetailsView->GetDetailsView()->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateLambda([]() { return false; }));


	ActivityList = SNew(SListView<FDataProviderActivityPtr>)
		.ListItemsSource(&FilteredActivities)
		.OnGenerateRow(this, &SDataProviderActivities::OnGenerateActivityRowWidget)
		.SelectionMode(ESelectionMode::Single)
		.OnSelectionChanged(this, &SDataProviderActivities::OnListViewSelectionChanged)
		//.AllowOverscroll(EAllowOverscroll::No)
		.HeaderRow
		(
			SNew(SHeaderRow)
			+ SHeaderRow::Column(DataProviderActivitiesListView::HeaderIdName_Timecode)
			.FillWidth(15.f)
			.DefaultLabel(LOCTEXT("HeaderName_Timecode", "Timecode"))
			+ SHeaderRow::Column(DataProviderActivitiesListView::HeaderIdName_StageName)
			.FillWidth(25.f)
			.DefaultLabel(LOCTEXT("HeaderName_StageName", "Stage Name"))
			+ SHeaderRow::Column(DataProviderActivitiesListView::HeaderIdName_Type)
			.FillWidth(25.f)
			.DefaultLabel(LOCTEXT("HeaderName_Type", "Type"))
			+ SHeaderRow::Column(DataProviderActivitiesListView::HeaderIdName_Description)
			.FillWidth(35.f)
			.DefaultLabel(LOCTEXT("HeaderName_Description", "Description"))
		);

	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(Orient_Horizontal)
		// List view
		+ SSplitter::Slot()
		.Value(0.75f)
		[
			SNew(SVerticalBox)
			// Filter
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(1.f, 1.f, 1.f, 1.f)
			[
				SAssignNew(ActivityFilter, SDataProviderActivityFilter, Session)	
				.OnActivityFilterChanged(FSimpleDelegate::CreateSP(this, &SDataProviderActivities::OnActivityFilterChanged))
			]
			// Activities
			+ SVerticalBox::Slot()
			.FillHeight(.8f)
			.VAlign(VAlign_Fill)
			.Padding(10.f, 0.f, 10.f, 10.f)
			[
				ActivityList.ToSharedRef()
			]
		]
		+ SSplitter::Slot()
		.Value(.25f)
		[
			SNew(SBorder)
			.VAlign(VAlign_Fill)
			[
				StructureDetailsView->GetWidget().ToSharedRef()
			]
		]
	];

	RequestRebuild();
}

SDataProviderActivities::~SDataProviderActivities()
{
}


void SDataProviderActivities::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bRebuildRequested)
	{
		bRebuildRequested = false;
		Activities.Empty();
		FilteredActivities.Empty();
		ReloadActivityHistory();
		ActivityList->RebuildList();
	}

	if (bRefreshRequested && FramesSinceLastListRefresh > CVarStageMonitorMinFramesBetweenRefresh.GetValueOnAnyThread())
	{
		FramesSinceLastListRefresh = 0;
		bRefreshRequested = false;
		ActivityList->RequestListRefresh();
	}
		
	Super::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	
	FramesSinceLastListRefresh++;
}


void SDataProviderActivities::RequestRebuild()
{
	bRebuildRequested = true;
}

void SDataProviderActivities::RequestRefresh()
{
	bRefreshRequested = true;
}

void SDataProviderActivities::RefreshMonitorSession(const TWeakPtr<IStageMonitorSession>& NewSession)
{
	ActivityFilter->RefreshMonitorSession(NewSession);
	AttachToMonitorSession(NewSession);
	RequestRebuild();
}

TSharedRef<ITableRow> SDataProviderActivities::OnGenerateActivityRowWidget(FDataProviderActivityPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<SDataProviderActivitiesTableRow> Row = SNew(SDataProviderActivitiesTableRow, OwnerTable, Session)
		.Item(InItem);
	return Row;
}

void SDataProviderActivities::OnListViewSelectionChanged(FDataProviderActivityPtr InActivity, ESelectInfo::Type SelectInfo)
{
	if (InActivity.IsValid())
	{
		StructureDetailsView->SetStructureData(InActivity->Data);
	}
	else
	{
		StructureDetailsView->SetStructureData(nullptr);
	}
}

void SDataProviderActivities::OnNewStageActivity(FDataProviderActivityPtr NewActivity)
{
	//Make new row data
	Activities.Add(NewActivity);
	if (ActivityFilter->GetActivityFilter().DoesItPass(NewActivity))
	{
		InsertActivity(NewActivity);
		RequestRefresh();
	}
}

void SDataProviderActivities::InsertActivity(FDataProviderActivityPtr Activity)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(StageMonitor::InsertActivity);

	//We might receive messages out of order, make sure we're displaying them in order of source timecode
	const FStageProviderMessage* ThisActivity = reinterpret_cast<const FStageProviderMessage*>(Activity->Data->GetStructMemory());
	const double NewFrameSeconds = ThisActivity->FrameTime.AsSeconds();
	const double NewFrameDay = FMath::Floor(ThisActivity->DateTime.GetJulianDay());

	int32 EntryIndex = 0;

	// If an activity has a negative timecode, just add it to the top of the list to avoid comparing against all the entries in the list.
	if (NewFrameSeconds >= 0)
	{
		for (; EntryIndex < FilteredActivities.Num(); ++EntryIndex)
		{
			const FStageProviderMessage* ThisEntry = reinterpret_cast<const FStageProviderMessage*>(FilteredActivities[EntryIndex]->Data->GetStructMemory());
			const double ThisFrameSeconds = ThisEntry->FrameTime.AsSeconds();
			const double ThisFrameDay = FMath::Floor(ThisEntry->DateTime.GetJulianDay());

			if (NewFrameSeconds >= ThisFrameSeconds || NewFrameDay > ThisFrameDay)
			{
				break;
			}
		}
	}

	FilteredActivities.Insert(Activity, EntryIndex);
}

void SDataProviderActivities::OnActivityFilterChanged()
{
	//When filtering has changed, update the current filtered list based on the full list
	FilteredActivities.Reset(Activities.Num());

	auto FrameTimeProjection = [](FDataProviderActivityPtr ActivityPtr)
	{
		const FStageProviderMessage* ThisEntry = reinterpret_cast<const FStageProviderMessage*>(ActivityPtr->Data->GetStructMemory());
		return ThisEntry->FrameTime.AsSeconds();
	};
	
	Algo::SortBy(Activities, FrameTimeProjection, TGreater<>());
	ActivityFilter->GetActivityFilter().FilterActivities(Activities, FilteredActivities);
	ActivityList->ScrollToTop();

	// Request a refresh to update the view
	RequestRefresh();
}

void SDataProviderActivities::ReloadActivityHistory()
{
	FilteredActivities.Reset();

	if (TSharedPtr<IStageMonitorSession> SessionPtr = Session.Pin())
	{
		TArray<TSharedPtr<FStageDataEntry>> CurrentActivities;
		SessionPtr->GetAllEntries(CurrentActivities);

		for (const TSharedPtr<FStageDataEntry>& Entry : CurrentActivities)
		{
			OnNewStageActivity(Entry);
		}
	}

	RequestRefresh();
}

void SDataProviderActivities::OnStageDataCleared()
{
	RequestRebuild();
}

void SDataProviderActivities::AttachToMonitorSession(const TWeakPtr<IStageMonitorSession>& NewSession)
{
	if (NewSession != Session)
	{
		if (TSharedPtr<IStageMonitorSession> SessionPtr = Session.Pin())
		{
			SessionPtr->OnStageSessionNewDataReceived().RemoveAll(this);
			SessionPtr->OnStageSessionDataCleared().RemoveAll(this);
		}

		Session = NewSession;

		if (TSharedPtr<IStageMonitorSession> SessionPtr = Session.Pin())
		{
			SessionPtr->OnStageSessionNewDataReceived().AddSP(this, &SDataProviderActivities::OnNewStageActivity);
			SessionPtr->OnStageSessionDataCleared().AddSP(this, &SDataProviderActivities::OnStageDataCleared);
		}
	}
}

/**
 * SDataProviderActivitiesTableRow
 */
void SDataProviderActivitiesTableRow::Construct(const FArguments & InArgs, const TSharedRef<STableViewBase> & InOwerTableView, TWeakPtr<IStageMonitorSession> InSession)
{
	Session = InSession;
	Item = InArgs._Item;
	check(Item.IsValid());

	Super::FArguments Arg;
	
	if (TSharedPtr<IStageMonitorSession> SessionPtr = Session.Pin())
	{
		if (Item.IsValid())
		{
			FStageProviderMessage* Data = reinterpret_cast<FStageProviderMessage*>(Item->Data->GetStructMemory());

			FStageSessionProviderEntry Provider;
			if (SessionPtr->GetProvider(Data->Identifier, Provider))
			{
				Descriptor = Provider.Descriptor;
				if (SessionPtr->IsTimePartOfCriticalState(Data->FrameTime.AsSeconds()))
				{
					Arg.Style(FStageMonitorEditorStyle::Get(), "TableView.CriticalStateRow");
				}
			}
		}
	}

	Super::Construct(Arg, InOwerTableView);
}

TSharedRef<SWidget> SDataProviderActivitiesTableRow::GenerateWidgetForColumn(const FName & ColumnName)
{
	if (DataProviderActivitiesListView::HeaderIdName_Timecode == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SDataProviderActivitiesTableRow::GetTimecode)
			];
	}
	if (DataProviderActivitiesListView::HeaderIdName_StageName == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SDataProviderActivitiesTableRow::GetStageName)
			];
	}
	if (DataProviderActivitiesListView::HeaderIdName_Type == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SDataProviderActivitiesTableRow::GetMessageType)
			];
	}
	if (DataProviderActivitiesListView::HeaderIdName_Description == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SDataProviderActivitiesTableRow::GetDescription)
			];
	}

	return SNullWidget::NullWidget;
}

FText SDataProviderActivitiesTableRow::GetTimecode() const
{	
	if (Item.IsValid())
	{
		check(Item->Data->GetStruct()->IsChildOf(FStageProviderMessage::StaticStruct()));
		FStageProviderMessage* Data = reinterpret_cast<FStageProviderMessage*>(Item->Data->GetStructMemory());
		if (Data->FrameTime.AsSeconds() >= 0)
		{
			return FText::FromString(Data->FrameTime.ToTimecode().ToString());
		}
		else
		{
			return LOCTEXT("InvalidTimecode", "Invalid Timecode");
		}
	}

	return FText::GetEmpty();
}

FText SDataProviderActivitiesTableRow::GetStageName() const
{
	if (Item.IsValid())
	{
		return FText::FromName(Descriptor.FriendlyName);
	}

	return FText::GetEmpty();
}

FText SDataProviderActivitiesTableRow::GetMessageType() const
{
	if (Item.IsValid())
	{
		check(Item->Data->GetStruct()->IsChildOf(FStageProviderMessage::StaticStruct()));
		return Item->Data->GetStruct()->GetDisplayNameText();
	}

	return FText::GetEmpty();
}

FText SDataProviderActivitiesTableRow::GetDescription() const
{
	if (Item.IsValid())
	{
		check(Item->Data->GetStruct()->IsChildOf(FStageProviderMessage::StaticStruct()));
		FStageProviderMessage* Data = reinterpret_cast<FStageProviderMessage*>(Item->Data->GetStructMemory());
		return FText::FromString(Data->ToString());
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE /*SDataProviderActivities*/
