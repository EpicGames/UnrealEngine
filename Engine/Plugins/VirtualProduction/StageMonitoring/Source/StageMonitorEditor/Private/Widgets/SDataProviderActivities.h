// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"


#include "StageMessages.h"

class STableViewBase;

class IStageMonitor;
class SStageMonitorPanel;
class IStageMonitorSession;
class IStructureDetailsView;
class FStructOnScope;
class SDataProviderActivityFilter;
struct FStageDataEntry;

using FDataProviderActivityPtr = TSharedPtr<FStageDataEntry>;


/**
 *
 */
class SDataProviderActivitiesTableRow : public SMultiColumnTableRow<FDataProviderActivityPtr>
{
	using Super = SMultiColumnTableRow<FDataProviderActivityPtr>;

public:
	SLATE_BEGIN_ARGS(SDataProviderActivitiesTableRow) { }
	SLATE_ARGUMENT(FDataProviderActivityPtr, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwerTableView, TWeakPtr<IStageMonitorSession> InSession);
	

private:
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	FText GetTimecode() const;
	FText GetStageName() const;
	FText GetMessageType() const;
	FText GetDescription() const;

private:
	FDataProviderActivityPtr Item;
	FStageInstanceDescriptor Descriptor;
	TWeakPtr<IStageMonitorSession> Session;
};


/**
 *
 */
class SDataProviderActivities : public SCompoundWidget
{
	using Super = SCompoundWidget;

public:
	SLATE_BEGIN_ARGS(SDataProviderActivities) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<SStageMonitorPanel> OwnerPanel, const TWeakPtr<IStageMonitorSession>& InSession);
	virtual ~SDataProviderActivities();

	//~ Begin SCompoundWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~End SCompoundWidget interface

	/** Request a full rebuild of the list entries */
	void RequestRebuild();

	/** Request a refresh of the list widget. */
	void RequestRefresh();

	/** Refreshes session used to fetch data */
	void RefreshMonitorSession(const TWeakPtr<IStageMonitorSession>& NewSession);

private:
	TSharedRef<ITableRow> OnGenerateActivityRowWidget(FDataProviderActivityPtr Item, const TSharedRef<STableViewBase>& OwnerTable);
	void OnListViewSelectionChanged(FDataProviderActivityPtr InActivity, ESelectInfo::Type SelectInfo);
	void OnNewStageActivity(FDataProviderActivityPtr NewActivity);
	void InsertActivity(FDataProviderActivityPtr Activity);
	void OnActivityFilterChanged();
	void ReloadActivityHistory();
	void OnStageDataCleared();

	/** Bind useful delegate to the active session */
	void AttachToMonitorSession(const TWeakPtr<IStageMonitorSession>& NewSession);

private:
	TWeakPtr<SStageMonitorPanel> OwnerPanel;
	TWeakPtr<IStageMonitorSession> Session;
	TSharedPtr<SListView<FDataProviderActivityPtr>> ActivityList;

	TArray<FDataProviderActivityPtr> Activities;
	TArray<FDataProviderActivityPtr> FilteredActivities;
	TSharedPtr<IStructureDetailsView> StructureDetailsView;
	TSharedPtr<SDataProviderActivityFilter> ActivityFilter;

	int32 FramesSinceLastListRefresh = 0;
	
	bool bRebuildRequested = false;
	bool bRefreshRequested = false;
};

