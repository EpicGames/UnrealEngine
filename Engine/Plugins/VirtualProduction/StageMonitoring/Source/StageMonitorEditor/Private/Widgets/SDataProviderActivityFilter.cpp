// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDataProviderActivityFilter.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IStageMonitorSession.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/StructOnScope.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SNumericEntryBox.h"

#define LOCTEXT_NAMESPACE "SDataProviderActivityFilter"


static TAutoConsoleVariable<int32> CVarStageMonitorDefaultMaxMessageAge(TEXT("StageMonitor.DefaultMaxMessageAge"), 30, TEXT("The default value in minutes for the maximum age for which to display messages in the data monitor."));


FDataProviderActivityFilter::FDataProviderActivityFilter(TWeakPtr<IStageMonitorSession> InSession)
	: Session(MoveTemp(InSession))
{
	FilterSettings.MaxMessageAgeInMinutes = CVarStageMonitorDefaultMaxMessageAge.GetValueOnAnyThread();
	//Default to filter out period message types
	for (TObjectIterator<UScriptStruct> It; It; ++It)
	{
		UScriptStruct* Struct = *It;
		UScriptStruct* PeriodicMessageStruct = FStageProviderPeriodicMessage::StaticStruct();
		const bool bIsPeriodMessageStruct = PeriodicMessageStruct && Struct->IsChildOf(PeriodicMessageStruct) && (Struct != PeriodicMessageStruct);
		if (bIsPeriodMessageStruct)
		{
			FilterSettings.ExistingPeriodicTypes.Add(Struct);
			FilterSettings.RestrictedTypes.Add(Struct);
		}
	}
}

bool FDataProviderActivityFilter::DoesItPass(TSharedPtr<FStageDataEntry>& Entry) const
{
	return FilterActivity(Entry) == EFilterResult::Pass ? true : false;
}

void FDataProviderActivityFilter::FilterActivities(const TArray<TSharedPtr<FStageDataEntry>>& InUnfilteredActivities, TArray<TSharedPtr<FStageDataEntry>>& OutFilteredActivities) const
{
	for (const TSharedPtr<FStageDataEntry>& Entry : InUnfilteredActivities)
	{
		const EFilterResult Result = FilterActivity(Entry);
		if (Result == EFilterResult::Pass)
		{
			OutFilteredActivities.Add(Entry);
		}
		else if (Result == EFilterResult::FailMaxAge)
		{
			// Since activities are sorted by time code, don't bother filtering the rest of the activities since they are older.
			break;
		}
	}
}

FDataProviderActivityFilter::EFilterResult FDataProviderActivityFilter::FilterActivity(const TSharedPtr<FStageDataEntry>& Entry) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(StageMonitor::FilterActivity);
	const TSharedPtr<IStageMonitorSession> SessionPtr = Session.Pin();
	
	if (!SessionPtr.IsValid() || !Entry.IsValid() || !Entry->Data.IsValid() || Entry->Data->GetStructMemory() == nullptr)
	{
		return EFilterResult::InvalidEntry;
	}

	if (FilterSettings.RestrictedTypes.Contains(Cast<UScriptStruct>(Entry->Data->GetStruct())))
	{
		return EFilterResult::FailRestrictedTypes;
	}

	const FStageDataBaseMessage* Message = reinterpret_cast<const FStageDataBaseMessage*>(Entry->Data->GetStructMemory());
	const FGuid ProviderIdentifier = Message->Identifier;

	FStageSessionProviderEntry ProviderEntry;
	if (SessionPtr->GetProvider(ProviderIdentifier, ProviderEntry))
	{
		const FName FriendlyName = ProviderEntry.Descriptor.FriendlyName;
		if (FilterSettings.RestrictedProviders.ContainsByPredicate([FriendlyName](const FName& Other) { return Other == FriendlyName; }))
		{
			return EFilterResult::FailRestrictedProviders;
		}

		if (FilterSettings.RestrictedRoles.Num() > 0)
		{
			if (CachedRoleStringToArray.Contains(ProviderEntry.Descriptor.RolesStringified) == false)
			{
				TArray<FString> RoleArray;
				ProviderEntry.Descriptor.RolesStringified.ParseIntoArray(RoleArray, TEXT(","));
				TArray<FName> ToCache;
				ToCache.Reserve(RoleArray.Num());
				for (const FString& RoleString : RoleArray)
				{
					ToCache.Add(FName(RoleString));
				}

				CachedRoleStringToArray.Add(ProviderEntry.Descriptor.RolesStringified, ToCache);
			}

			bool bHasOneValidRole = false;
			const TArray<FName>& CachedRoles = CachedRoleStringToArray.FindChecked(ProviderEntry.Descriptor.RolesStringified);
			for (FName Role : CachedRoles)
			{
				if (!FilterSettings.RestrictedRoles.Contains(Role))
				{
					bHasOneValidRole = true;
					break;
				}
			}

			if (bHasOneValidRole == false)
			{
				return EFilterResult::FailRestrictedRoles;
			}
		}
	}

	if (FilterSettings.bEnableTimeFilter)
	{
		const double AgeInMinutes = (FApp::GetCurrentTime() - Entry->MessageTime) / 60.0;
		if (FMath::Max<uint32>(0, FMath::RoundToInt(AgeInMinutes)) >= FilterSettings.MaxMessageAgeInMinutes)
		{
			return EFilterResult::FailMaxAge;
		}
	}

	//If we're not filtering based on critical sources, we're done here
	if (!FilterSettings.bEnableCriticalStateFilter)
	{
		return EFilterResult::Pass;
	}
	
	const FStageProviderMessage* ProviderMessage = reinterpret_cast<const FStageProviderMessage*>(Entry->Data->GetStructMemory());
	const TArray<FName> Sources = SessionPtr->GetCriticalStateSources(ProviderMessage->FrameTime.AsSeconds());
	for (const FName& Source : Sources)
	{
		if (FilterSettings.RestrictedSources.Contains(Source))
		{
			return EFilterResult::Pass;
		}
	}
	
	return EFilterResult::FailRestrictedCriticalState;
}

SDataProviderActivityFilter::~SDataProviderActivityFilter()
{
	if (UObjectInitialized())
	{
		SaveSettings();
	}
}

void SDataProviderActivityFilter::Construct(const FArguments& InArgs, const TWeakPtr<IStageMonitorSession>& InSession)
{
	AttachToMonitorSession(InSession);

	CurrentFilter = MakeUnique<FDataProviderActivityFilter>(Session);

	OnActivityFilterChanged = InArgs._OnActivityFilterChanged;
	
	ChildSlot
	[
		SNew(SHorizontalBox)
		// Data type filter
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SComboButton)
			.ComboButtonStyle(FAppStyle::Get(), "GenericFilters.ComboButtonStyle")
			.ForegroundColor(FLinearColor::White)
			.ToolTipText(LOCTEXT("AddFilterToolTip", "Configure activity filter."))
			.OnGetMenuContent(this, &SDataProviderActivityFilter::MakeAddFilterMenu)
			.HasDownArrow(true)
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "GenericFilters.TextStyle")
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
					.Text(FText::FromString(FString(TEXT("\xf0b0"))) /*fa-filter*/)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2, 0, 0, 0)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "GenericFilters.TextStyle")
					.Text(LOCTEXT("Filters", "Filters"))
				]
			]
		]
		// Search (to do)
		+ SHorizontalBox::Slot()
		.Padding(4, 1, 0, 0)
		.FillWidth(1.0f)
		[
			SNew(STextBlock)
		]
	];

	LoadSettings();
}

void SDataProviderActivityFilter::RefreshMonitorSession(const TWeakPtr<IStageMonitorSession>& NewSession)
{
	AttachToMonitorSession(NewSession);
}

void SDataProviderActivityFilter::ToggleProviderFilter(FName ProviderName)
{
	if (CurrentFilter->FilterSettings.RestrictedProviders.Contains(ProviderName))
	{
		CurrentFilter->FilterSettings.RestrictedProviders.RemoveSingle(ProviderName);
	}
	else
	{
		CurrentFilter->FilterSettings.RestrictedProviders.AddUnique(ProviderName);
	}

	OnActivityFilterChanged.ExecuteIfBound();
}

bool SDataProviderActivityFilter::IsProviderFiltered(FName ProviderName) const
{
	// The list contains types to filter. Inverse return value to display a more natural way of looking at filter choices
	return !CurrentFilter->FilterSettings.RestrictedProviders.Contains(ProviderName);
}

void SDataProviderActivityFilter::ToggleDataTypeFilter(UScriptStruct* Type)
{
	if(CurrentFilter->FilterSettings.RestrictedTypes.Contains(Type))
	{
		CurrentFilter->FilterSettings.RestrictedTypes.RemoveSingle(Type);
	}
	else
	{
		CurrentFilter->FilterSettings.RestrictedTypes.AddUnique(Type);
	}

	OnActivityFilterChanged.ExecuteIfBound();
}

bool SDataProviderActivityFilter::IsDataTypeFiltered(UScriptStruct* Type) const
{
	// The list contains types to filter. Inverse return value to display a more natural way of looking at filter choices
	return !CurrentFilter->FilterSettings.RestrictedTypes.Contains(Type);
}

void SDataProviderActivityFilter::ToggleCriticalStateSourceFilter(FName Source)
{
	if (CurrentFilter->FilterSettings.RestrictedSources.Contains(Source))
	{
		CurrentFilter->FilterSettings.RestrictedSources.RemoveSingle(Source);
	}
	else
	{
		CurrentFilter->FilterSettings.RestrictedSources.AddUnique(Source);
	}

	OnActivityFilterChanged.ExecuteIfBound();
}

bool SDataProviderActivityFilter::IsCriticalStateSourceFiltered(FName Source) const
{
	// The list contains sources to passing the filter.
	return CurrentFilter->FilterSettings.RestrictedSources.Contains(Source);
}

void SDataProviderActivityFilter::ToggleCriticalSourceEnabledFilter()
{
	CurrentFilter->FilterSettings.bEnableCriticalStateFilter = !CurrentFilter->FilterSettings.bEnableCriticalStateFilter;

	OnActivityFilterChanged.ExecuteIfBound();
}

void SDataProviderActivityFilter::ToggleTimeFilterEnabled(ECheckBoxState CheckboxState)
{
	CurrentFilter->FilterSettings.bEnableTimeFilter = !CurrentFilter->FilterSettings.bEnableTimeFilter;

	OnActivityFilterChanged.ExecuteIfBound();
}

bool SDataProviderActivityFilter::IsCriticalSourceFilteringEnabled() const
{
	return CurrentFilter->FilterSettings.bEnableCriticalStateFilter;
}

bool SDataProviderActivityFilter::IsTimeFilterEnabled() const
{
	return CurrentFilter->FilterSettings.bEnableTimeFilter;
}

TSharedRef<SWidget> SDataProviderActivityFilter::MakeAddFilterMenu()
{
	// generate menu
	const bool bShouldCloseWindowAfterClosing = false;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterClosing, nullptr);

	MenuBuilder.BeginSection("ProviderActivityFilter", LOCTEXT("ProviderActivityFilterMenu", "Provider Activity Filters"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("FilterMessageType", "Message Type"),
			LOCTEXT("FilterMessageTypeToolTip", "Filters based on the type of the message"),
			FNewMenuDelegate::CreateSP(this, &SDataProviderActivityFilter::CreateMessageTypeFilterMenu),
			FUIAction(),
			NAME_None,
			EUserInterfaceActionType::None
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("FilterProvider", "Provider"),
			LOCTEXT("FilterProviderToolTip", "Filters based on the provider"),
			FNewMenuDelegate::CreateSP(this, &SDataProviderActivityFilter::CreateProviderFilterMenu),
			FUIAction(),
			NAME_None,
			EUserInterfaceActionType::None
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("FilterRoles", "Role"),
			LOCTEXT("FilterRolesToolTip", "Filters based on the provider role"),
			FNewMenuDelegate::CreateSP(this, &SDataProviderActivityFilter::CreateRoleFilterMenu),
			FUIAction(),
			NAME_None,
			EUserInterfaceActionType::None
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("FilterCriticalSources", "Critical state source"),
			LOCTEXT("FilterCriticalSourcesTooltip", "Filters based on critical state sources"),
			FNewMenuDelegate::CreateSP(this, &SDataProviderActivityFilter::CreateCriticalStateSourceFilterMenu),
			FUIAction
			(
				FExecuteAction::CreateSP(this, &SDataProviderActivityFilter::ToggleCriticalSourceEnabledFilter)
				, FCanExecuteAction()
				, FIsActionChecked::CreateSP(this, &SDataProviderActivityFilter::IsCriticalSourceFilteringEnabled)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("FilterBasedOnTime", "Time"),
			LOCTEXT("FilterRecentMessagesTooltip", "Filters based on the messages' age"),
			FNewMenuDelegate::CreateSP(this, &SDataProviderActivityFilter::CreateTimeFilterMenu),
			FUIAction(),
			NAME_None,
			EUserInterfaceActionType::None
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SDataProviderActivityFilter::CreateMessageTypeFilterMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("AvailableProviderMessageTypes", LOCTEXT("AvailableDataTypes", "Provider Message Types"));
	{
		for (TObjectIterator<UScriptStruct> It; It; ++It)
		{
			UScriptStruct* Struct = *It;
			const UScriptStruct* BasePeriodicMessage = FStageProviderPeriodicMessage::StaticStruct();
			const UScriptStruct* BaseEventMessage = FStageProviderEventMessage::StaticStruct();
			const bool bIsValidMessageStruct = (BasePeriodicMessage && Struct->IsChildOf(BasePeriodicMessage) && (Struct != BasePeriodicMessage))
												|| (BaseEventMessage && Struct->IsChildOf(BaseEventMessage) && (Struct != BaseEventMessage));
											 
			if (bIsValidMessageStruct)
			{
				MenuBuilder.AddMenuEntry
				(
					Struct->GetDisplayNameText(), //Label
					Struct->GetToolTipText(), //Tooltip
					FSlateIcon(),
					FUIAction
					(
						FExecuteAction::CreateSP(this, &SDataProviderActivityFilter::ToggleDataTypeFilter, Struct)
						, FCanExecuteAction()
						, FIsActionChecked::CreateSP(this, &SDataProviderActivityFilter::IsDataTypeFiltered, Struct)
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
				);
			}
		}
	}
	MenuBuilder.EndSection();
}

void SDataProviderActivityFilter::CreateProviderFilterMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("AvailableProviders", LOCTEXT("AvailableProviders", "Providers"));
	{
		if (TSharedPtr<IStageMonitorSession> SessionPtr = Session.Pin())
		{
			const auto CreateMenuEntry = [&MenuBuilder, this](const FStageSessionProviderEntry& Provider)
			{
				const FString ProviderName = FString::Printf(TEXT("%s"), *Provider.Descriptor.FriendlyName.ToString());
				MenuBuilder.AddMenuEntry
				(
					FText::FromString(ProviderName), //Label
					FText::GetEmpty(), //Tooltip
					FSlateIcon(),
					FUIAction
					(
						FExecuteAction::CreateSP(this, &SDataProviderActivityFilter::ToggleProviderFilter, Provider.Descriptor.FriendlyName)
						, FCanExecuteAction()
						, FIsActionChecked::CreateSP(this, &SDataProviderActivityFilter::IsProviderFiltered, Provider.Descriptor.FriendlyName)
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
				);
			};

			for (const FStageSessionProviderEntry& Provider : SessionPtr->GetProviders())
			{
				CreateMenuEntry(Provider);
			}
			
			for (const FStageSessionProviderEntry& Provider : SessionPtr->GetClearedProviders())
			{
				CreateMenuEntry(Provider);
			}
		}
	}
	MenuBuilder.EndSection();
}

void SDataProviderActivityFilter::ToggleRoleFilter(FName RoleName)
{
	if (CurrentFilter->FilterSettings.RestrictedRoles.Contains(RoleName))
	{
		CurrentFilter->FilterSettings.RestrictedRoles.RemoveSingle(RoleName);
	}
	else
	{
		CurrentFilter->FilterSettings.RestrictedRoles.AddUnique(RoleName);
	}

	OnActivityFilterChanged.ExecuteIfBound();
}

bool SDataProviderActivityFilter::IsRoleFiltered(FName RoleName) const
{
	// The list contains types to filter. Inverse return value to display a more natural way of looking at filter choices
	return !CurrentFilter->FilterSettings.RestrictedRoles.Contains(RoleName);
}

void SDataProviderActivityFilter::CreateRoleFilterMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("AvailableRoles", LOCTEXT("AvailableRoles", "Roles"));
	{
		if (const TSharedPtr<IStageMonitorSession> SessionPtr = Session.Pin())
		{
			//Build list of roles found in active and cleared providers
			TSet<FName> AvailableRoles;
			for (const FStageSessionProviderEntry& Provider : SessionPtr->GetProviders())
			{
				TArray<FString> ProviderRoles;
				Provider.Descriptor.RolesStringified.ParseIntoArray(ProviderRoles, TEXT(","));
				for (const FString& Role : ProviderRoles)
				{
					AvailableRoles.Add(*Role);
				}
			}

			for (const FStageSessionProviderEntry& Provider : SessionPtr->GetClearedProviders())
			{
				TArray<FString> ProviderRoles;
				Provider.Descriptor.RolesStringified.ParseIntoArray(ProviderRoles, TEXT(","));
				for (const FString& Role : ProviderRoles)
				{
					AvailableRoles.Add(*Role);
				}
			}

			for (FName Role : AvailableRoles)
			{
				MenuBuilder.AddMenuEntry
				(
					FText::FromName(Role), //Label
					FText::GetEmpty(), //Tooltip
					FSlateIcon(),
					FUIAction
					(
						FExecuteAction::CreateSP(this, &SDataProviderActivityFilter::ToggleRoleFilter, Role)
						, FCanExecuteAction()
						, FIsActionChecked::CreateSP(this, &SDataProviderActivityFilter::IsRoleFiltered, Role)
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
				);
			}
		}
	}
	MenuBuilder.EndSection();
}

void SDataProviderActivityFilter::CreateCriticalStateSourceFilterMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("CriticalStateSources", LOCTEXT("CriticalStateSources", "Critical state sources"));
	{
		if (TSharedPtr<IStageMonitorSession> SessionPtr = Session.Pin())
		{
			TArray<FName> Sources = SessionPtr->GetCriticalStateHistorySources();

			for (const FName& Source : Sources)
			{
				MenuBuilder.AddMenuEntry
				(
					FText::FromName(Source), //Label
					FText::GetEmpty(), //Tooltip
					FSlateIcon(),
					FUIAction
					(
						FExecuteAction::CreateSP(this, &SDataProviderActivityFilter::ToggleCriticalStateSourceFilter, Source)
						, FCanExecuteAction::CreateLambda([this]() { return CurrentFilter->FilterSettings.bEnableCriticalStateFilter; })
						, FIsActionChecked::CreateSP(this, &SDataProviderActivityFilter::IsCriticalStateSourceFiltered, Source)
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
				);
			}
		}

			}
	MenuBuilder.EndSection();
}

void SDataProviderActivityFilter::CreateTimeFilterMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("TimeFilter", LOCTEXT("TimeFilter", "Time filter"));
	{
		if (const TSharedPtr<IStageMonitorSession> SessionPtr = Session.Pin())
		{
			constexpr uint32 MaxAgeMaxSliderValue = 2880; // Default to a max slider value of 2 days
			
			MenuBuilder.AddWidget
			(
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SCheckBox)
					.OnCheckStateChanged(this, &SDataProviderActivityFilter::ToggleTimeFilterEnabled)
					.IsChecked(this, &SDataProviderActivityFilter::IsTimeFilterChecked)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2.f, 0.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MaxAgeLabel", "Max Age (min)"))
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SNumericEntryBox<uint32>)
					.MinDesiredValueWidth(100.f)
					.Value(this, &SDataProviderActivityFilter::GetMaxMessageAge)
					.AllowSpin(true)
					.MinValue(0)
					.MaxValue(TNumericLimits<uint32>::Max())
					.MinSliderValue(0)				
					.MaxSliderValue(MaxAgeMaxSliderValue) 
					.OnValueChanged(this, &SDataProviderActivityFilter::OnMaxMessageAgeChanged)
					.OnValueCommitted(this, &SDataProviderActivityFilter::OnMaxMessageAgeCommitted)
				],
				FText::GetEmpty(),
				true,
				false 
			);
		}
	}
	MenuBuilder.EndSection();
}

void SDataProviderActivityFilter::AttachToMonitorSession(const TWeakPtr<IStageMonitorSession>& NewSession)
{
	if (NewSession != Session)
	{
		Session = NewSession;
		CurrentFilter->Session = Session;
	}
}

ECheckBoxState SDataProviderActivityFilter::IsTimeFilterChecked() const
{
	return CurrentFilter->FilterSettings.bEnableTimeFilter ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

TOptional<uint32> SDataProviderActivityFilter::GetMaxMessageAge() const
{
	return CurrentFilter->FilterSettings.MaxMessageAgeInMinutes;
}

void SDataProviderActivityFilter::OnMaxMessageAgeChanged(uint32 NewMax)
{
	CurrentFilter->FilterSettings.MaxMessageAgeInMinutes = NewMax;
}

void SDataProviderActivityFilter::OnMaxMessageAgeCommitted(uint32 NewMax, ETextCommit::Type)
{
	CurrentFilter->FilterSettings.MaxMessageAgeInMinutes = NewMax;
	OnActivityFilterChanged.ExecuteIfBound();
}

void SDataProviderActivityFilter::LoadSettings()
{
	FString FoundIniSettings;
	if (GConfig->GetString(TEXT("StageMonitor"), TEXT("ActivityFilter"), FoundIniSettings, GEditorPerProjectIni))
	{
		FDataProviderActivityFilterSettings LoadedSettings;
		FDataProviderActivityFilterSettings::StaticStruct()->ImportText(*FoundIniSettings, &LoadedSettings, nullptr, PPF_None, GLog, TEXT("DataProviderActivityFilterSettings"));
		
		//Cleanup any types that don't exist anymore
		LoadedSettings.ExistingPeriodicTypes.RemoveAll([](const UScriptStruct* Other) { return Other == nullptr; });
		LoadedSettings.RestrictedTypes.RemoveAll([](const UScriptStruct* Other) { return Other == nullptr; });

		//Verify if there are new periodic types (filtered out by default) not found in the saved settings
		for (UScriptStruct* ExistingType : CurrentFilter->FilterSettings.ExistingPeriodicTypes)
		{
			if (ExistingType)
			{
				if (LoadedSettings.ExistingPeriodicTypes.Contains(ExistingType) == false)
				{
					LoadedSettings.ExistingPeriodicTypes.AddUnique(ExistingType);
				
					//We default to restrict periodic types so add it it by default
					LoadedSettings.RestrictedTypes.AddUnique(ExistingType);
				}
			}
		}

		CurrentFilter->FilterSettings = MoveTemp(LoadedSettings);
	}

	OnActivityFilterChanged.ExecuteIfBound();
}

void SDataProviderActivityFilter::SaveSettings()
{
	FString TextValue;
	FDataProviderActivityFilterSettings::StaticStruct()->ExportText(TextValue, &CurrentFilter->FilterSettings, &CurrentFilter->FilterSettings, nullptr, EPropertyPortFlags::PPF_None, nullptr);
	GConfig->SetString(TEXT("StageMonitor"), TEXT("ActivityFilter"), *TextValue, GEditorPerProjectIni);
}

#undef LOCTEXT_NAMESPACE
