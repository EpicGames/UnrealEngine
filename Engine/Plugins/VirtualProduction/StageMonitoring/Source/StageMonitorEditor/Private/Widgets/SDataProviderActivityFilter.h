// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"



#include "SDataProviderActivityFilter.generated.h"

namespace ETextCommit { enum Type : int; }
struct FStageDataEntry;

enum class ECheckBoxState : uint8;
class FStructOnScope;
class FMenuBuilder;
class IStageMonitorSession;


/** Filter settings used live and also load/saved to ini config */
USTRUCT()
struct FDataProviderActivityFilterSettings
{
	GENERATED_BODY()

	/** 
	 * Periodic message structure type. 
	 * Used to detect new types that would be filtered out automatically
	 */
	UPROPERTY()
	TArray<TObjectPtr<UScriptStruct>> ExistingPeriodicTypes;

	/** Message types that are filtered */
	UPROPERTY()
	TArray<TObjectPtr<UScriptStruct>> RestrictedTypes;

	/** Providers that are filtered using their friendly name */
	UPROPERTY()
	TArray<FName> RestrictedProviders;

	/** Roles that are filtered using their friendly name */
	UPROPERTY()
	TArray<FName> RestrictedRoles;

	/** Critical state sources that are filtered */
	UPROPERTY()
	TArray<FName> RestrictedSources;

	/** Global critical state filter state */
	UPROPERTY()
	bool bEnableCriticalStateFilter = false;

	/** Should time filtering (timecode age) be enabled */
	UPROPERTY()
	bool bEnableTimeFilter = true;

	/** How far back in time should we display messages */
	UPROPERTY()
	uint32 MaxMessageAgeInMinutes = 30;
};


/**
 *
 */
class FDataProviderActivityFilter
{
public:
	FDataProviderActivityFilter(TWeakPtr<IStageMonitorSession> InSession);

	/** Returns true if the entry passes the current filter */
	bool DoesItPass(TSharedPtr<FStageDataEntry>& Entry) const;
	void FilterActivities(const TArray<TSharedPtr<FStageDataEntry>>& InUnfilteredActivities, TArray<TSharedPtr<FStageDataEntry>>& OutFilteredActivities) const;

public:
	FDataProviderActivityFilterSettings FilterSettings;
	TWeakPtr<IStageMonitorSession> Session;

	/** Cache of stringified roles to array of roles to avoid parsing into array constantly */
	mutable TMap<FString, TArray<FName>> CachedRoleStringToArray;

private:
	enum class EFilterResult : uint8
	{
		InvalidEntry,
		FailRestrictedTypes,
		FailRestrictedProviders,
		FailRestrictedCriticalState,
		FailMaxAge,
		FailRestrictedRoles,
		Pass
	};
	
	EFilterResult FilterActivity(const TSharedPtr<FStageDataEntry>& Entries) const;
};


/**
 *
 */
class SDataProviderActivityFilter : public SCompoundWidget
{
	using Super = SCompoundWidget;

public:
	SLATE_BEGIN_ARGS(SDataProviderActivityFilter) {}
		SLATE_EVENT(FSimpleDelegate, OnActivityFilterChanged)
	SLATE_END_ARGS()

	~SDataProviderActivityFilter();
	void Construct(const FArguments& InArgs, const TWeakPtr<IStageMonitorSession>& InSession);

	/** Returns the activity filter */
	const FDataProviderActivityFilter& GetActivityFilter() const { return *CurrentFilter; }

	/** Refreshes session used to fetch data and update UI */
	void RefreshMonitorSession(const TWeakPtr<IStageMonitorSession>& NewSession);

private:

	/** Toggles state of provider filter */
	void ToggleProviderFilter(FName ProviderName);

	/** Returns true if provider is currently filtered out */
	bool IsProviderFiltered(FName ProviderName) const;

	/** Toggles state of data type filter */
	void ToggleDataTypeFilter(UScriptStruct* Type);
	
	/** Returns true if data type is currently filtered out */
	bool IsDataTypeFiltered(UScriptStruct* Type) const;

	/** Toggles state of critical state source filter */
	void ToggleCriticalStateSourceFilter(FName Source);

	/** Returns true if critical state source is currently filtered out */
	bool IsCriticalStateSourceFiltered(FName Source) const;

	/** Toggles whether critical source filtering is enabled or not */
	void ToggleCriticalSourceEnabledFilter();

	/** Toggle whether time filtering is enabled. */
	void ToggleTimeFilterEnabled(ECheckBoxState CheckboxState);

	/** Returns true if filtering by critical state source enabled */
	bool IsCriticalSourceFilteringEnabled() const;
	
	/** Returns true if filtering by message age is enabled. */
	bool IsTimeFilterEnabled() const;

	/** Create the AddFilter menu when combo button is clicked. Different filter types will be submenus */
	TSharedRef<SWidget> MakeAddFilterMenu();

	/** Creates the menu listing the different message types filter */
	void CreateMessageTypeFilterMenu(FMenuBuilder& MenuBuilder);

	/** Creates the menu listing the different providers filter */
	void CreateProviderFilterMenu(FMenuBuilder& MenuBuilder);

	/** Toggles state of role filter */
	void ToggleRoleFilter(FName RoleName);

	/** Returns true if role is currently filtered out */
	bool IsRoleFiltered(FName RoleName) const;
	
	/** Creates the menu listing the different roles filter */
	void CreateRoleFilterMenu(FMenuBuilder& MenuBuilder);

	/** Create the menu listing all critical state sources */
	void CreateCriticalStateSourceFilterMenu(FMenuBuilder& MenuBuilder);

	/** Create a menu listing the options for filtering message based on their age. */
	void CreateTimeFilterMenu(FMenuBuilder& MenuBuilder);

	/** Binds this widget to some session callbacks */
	void AttachToMonitorSession(const TWeakPtr<IStageMonitorSession>& NewSession);

	/** Returns whether the time filter checkbox should be checked or not. */
	ECheckBoxState IsTimeFilterChecked() const;

	/** Return the max age in minutes where messages should be filtered out. */
	TOptional<uint32> GetMaxMessageAge() const;

	/** Handle modifying the maximum message age. */
	void OnMaxMessageAgeChanged(uint32 NewMax);

	/** Handle setting the new maximum message age. */
	void OnMaxMessageAgeCommitted(uint32 NewMax, ETextCommit::Type);

	/** Load filtering settings from ini */
	void LoadSettings();

	/** Save filtering settings from ini */
	void SaveSettings();

private:

	/** Current state of the provider data filter */
	TUniquePtr<FDataProviderActivityFilter> CurrentFilter;

	/** Delegate fired when filter changed */
	FSimpleDelegate OnActivityFilterChanged;

	/** Weakptr to the current session we're sourcing info from */
	TWeakPtr<IStageMonitorSession> Session;
};

