// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDataProviderListView.h"

#include "Containers/StaticArray.h"
#include "Dom/JsonObject.h"
#include "EditorFontGlyphs.h"
#include "IStageMonitorSession.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/App.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/ScopeExit.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "StageMonitorEditorSettings.h"
#include "StageMonitorUtils.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/StructOnScope.h"
#include "Widgets/SDataProviderListView.h"
#include "Widgets/Views/SHeaderRow.h"


#define LOCTEXT_NAMESPACE "SDataProviderListView"

/**
 * This structure holds onto the definition and function callbacks for columns
 * in the Stage Data Monitor list. Each definition specifies the following
 *
 * - The label (display name)
 * - A flag to indicate if can be sorted
 * - A flag indicate if it is default visible
 * - A function callback when creating a new row
 * - A pointer to method on SDataProviderRow table to access FText result for data.
 * - A function callback that can be used to add additional slate args.
 * - An order id that specifies the order in the column is created.
 */
struct FColumnDefinition
{
	using FGenerateRowFuncType = TSharedRef<SWidget>(SDataProviderTableRow*);
	using FSetColumnArgumentsFuncType = void(SHeaderRow::FColumn::FArguments&);
	using FWidgetTextGetterType = FText(SDataProviderTableRow::*)() const;

	FColumnDefinition(TAttribute<FText> InLabel, bool bInSortable, bool bInIsVisible,
					  FGenerateRowFuncType* InGenerateRowFuncPtr,
					  FWidgetTextGetterType InTextGetterPtr,
					  FSetColumnArgumentsFuncType* InArgumentsColumnPtr = nullptr)
		: Label(MoveTemp(InLabel))
		, bSortable(bInSortable)
		, bIsVisible(bInIsVisible)
		, GenerateRowFuncPtr(InGenerateRowFuncPtr)
		, WidgetTextGetterPtr(InTextGetterPtr)
		, ColumnArgumentsFuncPtr(InArgumentsColumnPtr)
	{
		if (bTrackOrderIdCount)
		{
			OrderId = OrderIdCounter;
			OrderIdCounter++;
		}
	}

	FColumnDefinition() = delete;

	/** Label for the column */
	TAttribute<FText> Label;

	/** Can this column be sorted. */
	bool  bSortable = false;

	/** Is this column visible by default. */
	bool  bIsVisible = true;

	/** Function callback for generating a new row. */
	FGenerateRowFuncType* GenerateRowFuncPtr = nullptr;

	/** Pointer to member function that gets the FText for data */
	FWidgetTextGetterType WidgetTextGetterPtr = nullptr;

	/** A function callback for column arguments. */
	FSetColumnArgumentsFuncType* ColumnArgumentsFuncPtr = nullptr;

	/** Procedurally generated order id.*/
	uint32 OrderId = 0;

	static void BeginOrderIdCount()
	{
		bTrackOrderIdCount = true;
	}
	static void EndOrderIdCount()
	{
		bTrackOrderIdCount = false;
	}
private:

	static bool bTrackOrderIdCount;
	static uint32 OrderIdCounter;
};

uint32 FColumnDefinition::OrderIdCounter = 0;
bool FColumnDefinition::bTrackOrderIdCount = false;

namespace DataProviderListView
{

	const bool IsVisible = true;
	const bool IsNotVisible = false;
	const bool IsSortable = true;
	const bool IsNotSortable = false;

	namespace HeaderDef {

	const FName State = TEXT("State");
	const FName Status = TEXT("Status");
	const FName Timecode = TEXT("Timecode");
	const FName MachineName = TEXT("MachineName");
	const FName ProcessId = TEXT("ProcessId");
	const FName StageName = TEXT("StageName");
	const FName Roles = TEXT("Roles");
	const FName AverageFPS = TEXT("AverageFPS");
	const FName EstimatedMaxFPS = TEXT("EstimatedMaxFPS");
	const FName IdleTime = TEXT("IdleTime");
	const FName GameThreadTiming = TEXT("GameThreadTiming");
	const FName GameThreadWaitTiming = TEXT("GameThreadWaitTiming");
	const FName RenderThreadTiming = TEXT("RenderThreadTiming");
	const FName RenderThreadWaitTiming = TEXT("RenderThreadWaitTiming");
	const FName GPUTiming = TEXT("GPUTiming");
	const FName CPUMem = TEXT("CPUMem");
	const FName GPUMem = TEXT("GPUMem");
	const FName AssetsToCompile = TEXT("AssetsToCompile");

	const TMap<FName, FColumnDefinition> StaticColumnHeader = []() {
		FColumnDefinition::BeginOrderIdCount();
		TMap<FName, FColumnDefinition> Header = {
			{State,
			 {
				 LOCTEXT("HeaderName_State", "State"), IsSortable, IsVisible,
				 // Generate Column
				 [](SDataProviderTableRow* Provider) -> TSharedRef<SWidget>
				 {
					 return SNew(SHorizontalBox)
					 + SHorizontalBox::Slot()
					 .HAlign(HAlign_Center)
					 .VAlign(VAlign_Center)
					 .AutoWidth()
					 [
						 SNew(STextBlock)
							 .Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
							 .Text(Provider, &SDataProviderTableRow::GetStateGlyphs)
							 .ColorAndOpacity(Provider, &SDataProviderTableRow::GetStateColorAndOpacity)
					 ];
				 },
				 nullptr,
				 [](SHeaderRow::FColumn::FArguments& Args)
				 {
					 Args.FixedWidth(55.f);
				 }
			 }},
			{Status,
			 {
				 LOCTEXT("HeaderName_Status", "Status"), IsSortable, IsVisible,
				 // Generate Column
				 [](SDataProviderTableRow* Provider) -> TSharedRef<SWidget>
				 {
					 return SNew(SHorizontalBox)
					 + SHorizontalBox::Slot()
					 .HAlign(HAlign_Left)
					 .VAlign(VAlign_Center)
					 .AutoWidth()
					 [
						 SNew(STextBlock)
						 .Text(Provider, &SDataProviderTableRow::GetStatus)
						 .ToolTipText(Provider, &SDataProviderTableRow::GetStatusToolTip)
					 ];
				 },
				 &SDataProviderTableRow::GetStatus,
				 [](SHeaderRow::FColumn::FArguments& Args)
				 {
					 Args.FixedWidth(55.f);
				 }
			 }},
			{Timecode,
			 {
				 LOCTEXT("HeaderName_Timecode", "Timecode"), IsNotSortable, IsVisible,
				 [](SDataProviderTableRow* Provider) -> TSharedRef<SWidget>
				 {
					 return SNew(SHorizontalBox)
					 + SHorizontalBox::Slot()
					 .HAlign(HAlign_Left)
					 .VAlign(VAlign_Center)
					 .AutoWidth()
					 [
						 SNew(STextBlock)
						 .Text(Provider, &SDataProviderTableRow::GetTimecode)
					 ];
				 },
				 &SDataProviderTableRow::GetTimecode,
			 }
			},
			{MachineName,
			 {
				 LOCTEXT("HeaderName_MachineName", "Machine"), IsNotSortable, IsVisible,
				 [](SDataProviderTableRow* Provider) -> TSharedRef<SWidget>
				 {
					 return SNew(SHorizontalBox)
					 + SHorizontalBox::Slot()
					 .HAlign(HAlign_Left)
					 .VAlign(VAlign_Center)
					 .AutoWidth()
					 [
						 SNew(STextBlock)
						 .Text(Provider, &SDataProviderTableRow::GetMachineName)
					 ];
				 },
				 &SDataProviderTableRow::GetMachineName,
			 }
			},
			{ProcessId,
			 {
				 LOCTEXT("HeaderName_ProcessId", "Process Id"), IsSortable, IsNotVisible,
				 [](SDataProviderTableRow* Provider) -> TSharedRef<SWidget>
				 {
					 return SNew(SHorizontalBox)
					 + SHorizontalBox::Slot()
					 .HAlign(HAlign_Left)
					 .VAlign(VAlign_Center)
					 .AutoWidth()
					 [
						 SNew(STextBlock)
						 .Text(Provider, &SDataProviderTableRow::GetProcessId)
					 ];
				 },
				 &SDataProviderTableRow::GetProcessId,
			 }},
			{StageName,
			 {
				 LOCTEXT("HeaderName_StageName", "Stage Name"), IsSortable, IsVisible,
				 [](SDataProviderTableRow* Provider) -> TSharedRef<SWidget>
				 {
					 return SNew(SHorizontalBox)
					 + SHorizontalBox::Slot()
					 .HAlign(HAlign_Left)
					 .VAlign(VAlign_Center)
					 .AutoWidth()
					 [
						 SNew(STextBlock)
						 .Text(Provider, &SDataProviderTableRow::GetStageName)
					 ];
				 },
				 &SDataProviderTableRow::GetStageName,
			 }},
			{Roles,
			 {
				 LOCTEXT("HeaderName_Roles", "Roles"), IsSortable, IsNotVisible,
				 [](SDataProviderTableRow* Provider) -> TSharedRef<SWidget>
				 {
					 return SNew(SHorizontalBox)
					 + SHorizontalBox::Slot()
					 .HAlign(HAlign_Left)
					 .VAlign(VAlign_Center)
					 .AutoWidth()
					 [
						 SNew(STextBlock)
						 .Text(Provider, &SDataProviderTableRow::GetRoles)
					 ];
				 },
				 &SDataProviderTableRow::GetRoles,
			 }},
			{AverageFPS,
			 {
				 LOCTEXT("HeaderName_AverageFPS", "Average FPS"), IsNotSortable, IsVisible,
				 [](SDataProviderTableRow* Provider) -> TSharedRef<SWidget>
				 {
					 return SNew(SHorizontalBox)
					 + SHorizontalBox::Slot()
					 .HAlign(HAlign_Left)
					 .VAlign(VAlign_Center)
					 .AutoWidth()
					 [
						 SNew(STextBlock)
						 .Text(MakeAttributeSP(Provider, &SDataProviderTableRow::GetAverageFPS))
					 ];
				 },
				 &SDataProviderTableRow::GetAverageFPS,
			 }},
			{EstimatedMaxFPS,
			 {
				 LOCTEXT("HeaderName_EstimatedMaxFPS", "Estimated Max FPS"), IsNotSortable, IsVisible,
				 [](SDataProviderTableRow* Provider) -> TSharedRef<SWidget>
				 {
					 return SNew(SHorizontalBox)
					 + SHorizontalBox::Slot()
					 .HAlign(HAlign_Left)
					 .VAlign(VAlign_Center)
					 .AutoWidth()
					 [
						 SNew(STextBlock)
						 .Text(MakeAttributeSP(Provider, &SDataProviderTableRow::GetEstimatedMaxFPS))
					 ];
				 },
				 &SDataProviderTableRow::GetEstimatedMaxFPS,
			 }},
			{IdleTime,
			 {
				 LOCTEXT("HeaderName_IdleTime", "Idle Time (ms)"), IsNotSortable, IsNotVisible,
				 [](SDataProviderTableRow* Provider) -> TSharedRef<SWidget>
				 {
					 return SNew(SHorizontalBox)
					 + SHorizontalBox::Slot()
					 .HAlign(HAlign_Left)
					 .VAlign(VAlign_Center)
					 .AutoWidth()
					 [
						 SNew(STextBlock)
						 .Text(MakeAttributeSP(Provider, &SDataProviderTableRow::GetIdleTime))
					 ];
				 },
				 &SDataProviderTableRow::GetIdleTime,
			 }},
			{GameThreadTiming,
			 {
				 LOCTEXT("HeaderName_GameThread", "Game Thread (ms)"), IsNotSortable, IsVisible,
				 [](SDataProviderTableRow* Provider) -> TSharedRef<SWidget>
				 {
					 return SNew(SHorizontalBox)
					 + SHorizontalBox::Slot()
					 .HAlign(HAlign_Left)
					 .VAlign(VAlign_Center)
					 .AutoWidth()
					 [
						 SNew(STextBlock)
						 .Text(MakeAttributeSP(Provider, &SDataProviderTableRow::GetGameThreadTiming))
					 ];
				 },
				 &SDataProviderTableRow::GetGameThreadTiming,
			 }
			},
			{GameThreadWaitTiming,
			 {
				 LOCTEXT("HeaderName_GameWaitThread", "Game Wait Time (ms)"), IsNotSortable, IsNotVisible,
				 [](SDataProviderTableRow* Provider) -> TSharedRef<SWidget>
				 {
					 return SNew(SHorizontalBox)
					 + SHorizontalBox::Slot()
					 .HAlign(HAlign_Left)
					 .VAlign(VAlign_Center)
					 .AutoWidth()
					 [
						 SNew(STextBlock)
						 .Text(MakeAttributeSP(Provider, &SDataProviderTableRow::GetGameThreadWaitTiming))
					 ];
				 },
				 &SDataProviderTableRow::GetGameThreadWaitTiming,
			 }
			},
			{RenderThreadTiming,
			 {
				 LOCTEXT("HeaderName_RenderThread", "Render Thread (ms)"), IsNotSortable, IsVisible,
				 [](SDataProviderTableRow* Provider) -> TSharedRef<SWidget>
				 {
					 return SNew(SHorizontalBox)
					 + SHorizontalBox::Slot()
					 .HAlign(HAlign_Left)
					 .VAlign(VAlign_Center)
					 .AutoWidth()
					 [
						 SNew(STextBlock)
						 .Text(MakeAttributeSP(Provider, &SDataProviderTableRow::GetRenderThreadTiming))
					 ];
				 },
				 &SDataProviderTableRow::GetRenderThreadTiming,
			 }},
			{RenderThreadWaitTiming,
			 {
				 LOCTEXT("HeaderName_RenderWaitThread", "Render Wait Time (ms)"), IsNotSortable, IsNotVisible,
				 [](SDataProviderTableRow* Provider) -> TSharedRef<SWidget>
				 {
					 return SNew(SHorizontalBox)
					 + SHorizontalBox::Slot()
					 .HAlign(HAlign_Left)
					 .VAlign(VAlign_Center)
					 .AutoWidth()
					 [
						 SNew(STextBlock)
						 .Text(MakeAttributeSP(Provider, &SDataProviderTableRow::GetRenderThreadWaitTiming))
					 ];
				 },
				 &SDataProviderTableRow::GetRenderThreadWaitTiming,
			 }},
			{GPUTiming,
			 {
				 LOCTEXT("HeaderName_GPU", "GPU (ms)"), IsNotSortable, IsVisible,
				 [](SDataProviderTableRow* Provider) -> TSharedRef<SWidget>
				 {
					 return SNew(SHorizontalBox)
					 + SHorizontalBox::Slot()
					 .HAlign(HAlign_Left)
					 .VAlign(VAlign_Center)
					 .AutoWidth()
					 [
						 SNew(STextBlock)
						 .Text(MakeAttributeSP(Provider, &SDataProviderTableRow::GetGPUTiming))
					 ];
				 },
				 &SDataProviderTableRow::GetGPUTiming,
			 }},
			{CPUMem,
			 {
				 LOCTEXT("HeaderName_CPUMem", "CPU Memory"), IsNotSortable, IsNotVisible,
				 [](SDataProviderTableRow* Provider) -> TSharedRef<SWidget>
				 {
					 return SNew(SHorizontalBox)
					 + SHorizontalBox::Slot()
					 .HAlign(HAlign_Left)
					 .VAlign(VAlign_Center)
					 .AutoWidth()
					 [
						 SNew(STextBlock)
						 .Text(MakeAttributeSP(Provider, &SDataProviderTableRow::GetCPUMem))
					 ];
				 },
				 &SDataProviderTableRow::GetCPUMem,
			 }},
			{GPUMem,
			 {
				 LOCTEXT("HeaderName_GPUMem", "GPU Memory"), IsNotSortable, IsNotVisible,
				 [](SDataProviderTableRow* Provider) -> TSharedRef<SWidget>
				 {
					 return SNew(SHorizontalBox)
					 + SHorizontalBox::Slot()
					 .HAlign(HAlign_Left)
					 .VAlign(VAlign_Center)
					 .AutoWidth()
					 [
						 SNew(STextBlock)
						 .Text(MakeAttributeSP(Provider, &SDataProviderTableRow::GetGPUMem))
					 ];
				 },
				 &SDataProviderTableRow::GetGPUMem,
			 }
			},
			{AssetsToCompile,
			 {
				 LOCTEXT("HeaderName_Assets", "Assets To Compile"), IsNotSortable, IsNotVisible,
				 [](SDataProviderTableRow* Provider) -> TSharedRef<SWidget>
				 {
					 return SNew(SHorizontalBox)
					 + SHorizontalBox::Slot()
					 .HAlign(HAlign_Left)
					 .VAlign(VAlign_Center)
					 .AutoWidth()
					 [
						 SNew(STextBlock)
						 .Text(MakeAttributeSP(Provider, &SDataProviderTableRow::GetAssetsLeftToCompile))
					 ];
				 },
				 &SDataProviderTableRow::GetAssetsLeftToCompile
			 }
			}};
		FColumnDefinition::EndOrderIdCount();
		return Header;
	}();

	TArray<FName> GetSortedColumnNames()
	{
		// Declare the columns for the header row by referring to the defined table.
		TArray<FName> ColumnNames;
		Algo::Transform(StaticColumnHeader, ColumnNames,
			[](const TPair<FName,FColumnDefinition>& Item)
			{
				return Item.Get<0>();
			});

		ColumnNames.Sort(
			[](const FName& A, const FName& B)
			{
				return StaticColumnHeader.Find(A)->OrderId < StaticColumnHeader.Find(B)->OrderId;
			});

		return ColumnNames;
	}


	} // end namespace HeaderDef
}


/**
 * FDataProviderTableRowData
 */
struct FDataProviderTableRowData : TSharedFromThis<FDataProviderTableRowData>
{
	FDataProviderTableRowData(const FGuid& InIdentifier, const FStageInstanceDescriptor& InDescriptor, TWeakPtr<IStageMonitorSession> InSession)
		: Identifier(InIdentifier)
		, Descriptor(InDescriptor)
		, Session(InSession)
	{
	}

	/** Fetch latest information for the associated data provider */
	void UpdateCachedValues()
	{
		if (TSharedPtr<IStageMonitorSession> SessionPtr = Session.Pin())
		{
			CachedState = SessionPtr->GetProviderState(Identifier);

			TSharedPtr<FStageDataEntry> LatestData = SessionPtr->GetLatest(Identifier, FFramePerformanceProviderMessage::StaticStruct());
			if (LatestData.IsValid() && LatestData->Data.IsValid())
			{
				//Copy over this message data
				const FFramePerformanceProviderMessage* MessageData = reinterpret_cast<const FFramePerformanceProviderMessage*>(LatestData->Data->GetStructMemory());
				check(MessageData);
				CachedPerformanceData = *MessageData;
			}
		}
	}

public:

	/** Identifier and descriptor associated to this list entry */
	FGuid Identifier;
	FStageInstanceDescriptor Descriptor;

	/** Weak pointer to the session data */
	TWeakPtr<IStageMonitorSession> Session;

	/** Cached data for the frame performance of this provider */
	FFramePerformanceProviderMessage CachedPerformanceData;

	/** Cached state of this provider */
	EStageDataProviderState CachedState;
};


/**
 * SDataProviderTableRow
 */
void SDataProviderTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwerTableView)
{
	Item = InArgs._Item;
	check(Item.IsValid());

	Super::FArguments Arg;
	Arg._Padding = InArgs._Padding;
	Super::Construct(Arg, InOwerTableView);
}

TSharedRef<SWidget> SDataProviderTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	using namespace DataProviderListView::HeaderDef;
	const FColumnDefinition* ColumnDef = StaticColumnHeader.Find(ColumnName);
	if (ColumnDef && ColumnDef->GenerateRowFuncPtr)
	{
		TSharedRef<SWidget> Widget = ColumnDef->GenerateRowFuncPtr(this);
		Widget->SetToolTipText(
			MakeAttributeSP(this, &SDataProviderTableRow::GetToolTipText));
		return Widget;
	}
	return SNullWidget::NullWidget;
}

FText SDataProviderTableRow::GetToolTipText() const
{
	using MemberFunc = FText(SDataProviderTableRow::*)() const;
	using namespace DataProviderListView::HeaderDef;
	const SDataProviderTableRow& ThisClass = *this;
	FTextBuilder RunningTextObject;

	TArray<FName> Columns = GetSortedColumnNames();
	for (const FName& Column : Columns)
	{
		const FColumnDefinition* ColumnDef = StaticColumnHeader.Find(Column);
		if (ColumnDef && ColumnDef->WidgetTextGetterPtr)
		{
			FColumnDefinition::FWidgetTextGetterType GetterPtr = ColumnDef->WidgetTextGetterPtr;
			RunningTextObject.AppendLineFormat(LOCTEXT("StageMonitorProviderToolTip", "{0} : {1}"),
											   FText::FromString(Column.ToString()), (ThisClass.*GetterPtr)());
		}
	}
	return RunningTextObject.ToText();
}

FText SDataProviderTableRow::GetStateGlyphs() const
{
	if (Item->CachedPerformanceData.Status !=  EStageMonitorNodeStatus::Ready)
	{
		return FEditorFontGlyphs::Exclamation_Triangle;
	}
	return FEditorFontGlyphs::Circle;
}

FText SDataProviderTableRow::GetStatusToolTip() const
{
	return FText::AsCultureInvariant(Item->CachedPerformanceData.AssetInStatus);
}

FText SDataProviderTableRow::GetStatus() const
{
	switch(Item->CachedPerformanceData.Status)
	{
	case EStageMonitorNodeStatus::LoadingMap:
		return LOCTEXT("Status_LoadingMap", "Loading");
	case EStageMonitorNodeStatus::Ready:
		return LOCTEXT("Status_Ready", "Ready");
	case EStageMonitorNodeStatus::HotReload:
		return LOCTEXT("Status_HotReload", "Reloading");
	case EStageMonitorNodeStatus::AssetCompiling:
		return LOCTEXT("Status_AssetCompiling", "Asset Compiling");
	case EStageMonitorNodeStatus::Unknown:
		return LOCTEXT("Status_Unknown", "Unknown");
	default:
		break;
	};

	return LOCTEXT("Status_Undefined", "Undefined");
}

FSlateColor SDataProviderTableRow::GetStateColorAndOpacity() const
{
	if (Item->CachedState != EStageDataProviderState::Closed && Item->CachedPerformanceData.Status !=  EStageMonitorNodeStatus::Ready)
	{
		return FLinearColor::Yellow;
	}

	switch (Item->CachedState)
	{
		case EStageDataProviderState::Active:
		{
			return FLinearColor::Green;
		}
		case EStageDataProviderState::Inactive:
		{
			return FLinearColor::Yellow;
		}
		case EStageDataProviderState::Closed:
		default:
		{
			return FLinearColor::Red;
		}
	}
}

FString GetMemoryString( const uint64 Value )
{
	if (Value > 1024 * 1024)
	{
		// Display anything over a MB in GB format.
		return FString::Printf( TEXT( "%.2f GB" ), float( static_cast<float>(Value) / (1024.0 * 1024.0 * 1024.0) ) );
	}
	else if (Value > 1024)
	{
		// Anything more than a KB in MB.
		return FString::Printf( TEXT( "%.2f MB" ), float( static_cast<float>(Value) / (1024.0 * 1024.0) ) );
	}

	return FString::Printf( TEXT( "%.2f KB" ), float( static_cast<float>(Value) / (1024.0) ) );
}

FText SDataProviderTableRow::GetTimecode() const
{
	const FTimecode CachedTimecode = FTimecode::FromFrameNumber(Item->CachedPerformanceData.FrameTime.Time.FrameNumber, Item->CachedPerformanceData.FrameTime.Rate);
	return FText::FromString(CachedTimecode.ToString());
}

FText SDataProviderTableRow::GetMachineName() const
{
	return FText::FromString(Item->Descriptor.MachineName);
}

FText SDataProviderTableRow::GetProcessId() const
{
	return FText::AsNumber(Item->Descriptor.ProcessId);
}

FText SDataProviderTableRow::GetStageName() const
{
	return FText::FromName(Item->Descriptor.FriendlyName);
}

FText SDataProviderTableRow::GetRoles() const
{
	return FText::FromString(Item->Descriptor.RolesStringified);
}

FText SDataProviderTableRow::GetAverageFPS() const
{
	return FText::AsNumber(Item->CachedPerformanceData.AverageFPS);
}

FText SDataProviderTableRow::GetEstimatedMaxFPS() const
{
	const FFramePerformanceProviderMessage& Data = Item->CachedPerformanceData;
	const float TotalMS = Data.GameThreadMS + Data.GameThreadWaitMS;
	const float EstimatedGPUIdle = TotalMS > Data.GPU_MS ? (TotalMS - Data.GPU_MS) : TotalMS;
	const float EstimatedMinMS = TotalMS > EstimatedGPUIdle ? (TotalMS - EstimatedGPUIdle) : TotalMS;
	const float ClampedEstimatedMS = FMath::Clamp(EstimatedMinMS, Data.GPU_MS, EstimatedMinMS);
	const float EstimatedMaxFPS = ClampedEstimatedMS > 0 ? 1000.f / ClampedEstimatedMS : 0;
	return FText::AsNumber(EstimatedMaxFPS);
}

FText SDataProviderTableRow::GetIdleTime() const
{
	return FText::AsNumber(Item->CachedPerformanceData.IdleTimeMS);
}

FText SDataProviderTableRow::GetGameThreadTiming() const
{
	return FText::AsNumber(Item->CachedPerformanceData.GameThreadMS);
}

FText SDataProviderTableRow::GetGameThreadWaitTiming() const
{
	return FText::AsNumber(Item->CachedPerformanceData.GameThreadWaitMS);
}

FText SDataProviderTableRow::GetRenderThreadTiming() const
{
	return FText::AsNumber(Item->CachedPerformanceData.RenderThreadMS);
}

FText SDataProviderTableRow::GetRenderThreadWaitTiming() const
{
	return FText::AsNumber(Item->CachedPerformanceData.RenderThreadWaitMS);
}

FText SDataProviderTableRow::GetGPUTiming() const
{
	return FText::AsNumber(Item->CachedPerformanceData.GPU_MS);
}

FText SDataProviderTableRow::GetGPUMem() const
{
	return FText::FromString(GetMemoryString(Item->CachedPerformanceData.GPU_MEM));
}

FText SDataProviderTableRow::GetCPUMem() const
{
	return FText::FromString(GetMemoryString(Item->CachedPerformanceData.CPU_MEM));
}

FText SDataProviderTableRow::GetAssetsLeftToCompile() const
{
	return FText::AsNumber(Item->CachedPerformanceData.CompilationTasksRemaining);
}

namespace UE::StageMonitor::Private
{
	namespace JsonKeys
	{
		const FString ColumnId("ColumnID");
		const FString IsVisible("bIsVisible");
		const FString Values("Values");
	}
	void CaptureColumnVisibilityState(TSharedRef<SHeaderRow> HeaderRow)
	{
		TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		TArray<TSharedPtr<FJsonValue>> Array;
		for (const SHeaderRow::FColumn& Column : HeaderRow->GetColumns())
		{
			TSharedPtr<FJsonObject> ColumnObject = MakeShared<FJsonObject>();
			ColumnObject->SetStringField(JsonKeys::ColumnId, Column.ColumnId.ToString());
			ColumnObject->SetBoolField(JsonKeys::IsVisible, Column.bIsVisible);
			Array.Add(MakeShared<FJsonValueObject>(ColumnObject));
		}
		JsonObject->SetArrayField(JsonKeys::Values, Array);

		FString Snapshot;
		const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&Snapshot);
		FJsonSerializer::Serialize(JsonObject.ToSharedRef(), Writer);

		UStageMonitorEditorSettings* Settings = GetMutableDefault<UStageMonitorEditorSettings>();
		Settings->ColumnVisibilitySettings = Snapshot;
		Settings->SaveConfig();
	}

	TOptional<TPair<FString, bool>> GetColumnData(const TSharedPtr<FJsonValue>& ColumnValue)
	{
		TSharedPtr<FJsonObject>* PossibleColumnObject;
		if (!ColumnValue->TryGetObject(PossibleColumnObject))
		{
			return {};
		}
		const TSharedPtr<FJsonObject>& ColumnObject = *PossibleColumnObject;

		FString ColumnIdValue;
		bool bIsVisible;
		if (!ColumnObject->TryGetStringField(JsonKeys::ColumnId, ColumnIdValue)
			|| !ColumnObject->TryGetBoolField(JsonKeys::IsVisible, bIsVisible))
		{
			return {};
		}

		return {{ColumnIdValue, bIsVisible}};
	}

	void RestoreColumnVisibilityState(TSharedRef<SHeaderRow> HeaderRow)
	{
		const UStageMonitorEditorSettings* Settings = GetDefault<UStageMonitorEditorSettings>();
		TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		const TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Settings->ColumnVisibilitySettings);
		const bool bIsValid = FJsonSerializer::Deserialize(Reader, JsonObject);

		if (!bIsValid)
		{
			return;
		}

		const TArray<TSharedPtr<FJsonValue>> ColumnObjects = JsonObject->GetArrayField(JsonKeys::Values);
		for (const TSharedPtr<FJsonValue>& ColumnValue : ColumnObjects)
		{
			const TOptional<TPair<FString,bool>> Data = GetColumnData(ColumnValue);
			if (Data)
			{
				HeaderRow->SetShowGeneratedColumn(*Data->Key, Data->Value);
			}
		}
	}
}

/**
 * SDataProviderListView
 */
void SDataProviderListView::Construct(const FArguments& InArgs, const TWeakPtr<IStageMonitorSession>& InSession)
{
	using namespace DataProviderListView::HeaderDef;

	SortedColumnName = DataProviderListView::HeaderDef::StageName;
	SortMode = EColumnSortMode::Ascending;

	AttachToMonitorSession(InSession);

	// Construct the header row.
	HeaderRow = SNew(SHeaderRow)
		.OnHiddenColumnsListChanged_Lambda([this]()
			{
				if (!bUpdatingColumnVisibility)
				{
					UE::StageMonitor::Private::CaptureColumnVisibilityState(HeaderRow.ToSharedRef());
				}
			}
		)
		.CanSelectGeneratedColumn(true); //To show/hide columns

	Super::Construct
	(
		Super::FArguments()
		.ListItemsSource(&ListItemsSource)
		.SelectionMode(ESelectionMode::None)
		.OnGenerateRow(this, &SDataProviderListView::OnGenerateRow)
		.HeaderRow(HeaderRow)
	);

	TArray<FName> ColumnNames = GetSortedColumnNames();

	// Initialize the columns in the proper order.
	for (const FName& ColumnName : ColumnNames)
	{
		const FColumnDefinition* Column = StaticColumnHeader.Find(ColumnName);
		SHeaderRow::FColumn::FArguments Args = SHeaderRow::FColumn::FArguments()
			.ColumnId(ColumnName)
			.DefaultLabel(Column->Label)
			.HAlignHeader(HAlign_Center)
			.HAlignCell(HAlign_Center)
			;
		if (Column->bSortable)
		{
			Args.SortMode(this, &SDataProviderListView::GetColumnSortMode, ColumnName)
				.OnSort(this, &SDataProviderListView::OnColumnSortModeChanged);
		}
		if (Column->ColumnArgumentsFuncPtr)
		{
			Column->ColumnArgumentsFuncPtr(Args);
		}
		HeaderRow->AddColumn(Args);
	}

	{
		TGuardValue<bool> DoNotSave(bUpdatingColumnVisibility, true);
		SetDefaultColumnVisibilities();
		UE::StageMonitor::Private::RestoreColumnVisibilityState(HeaderRow.ToSharedRef());
	}

	RebuildDataProviderList();
}

void SDataProviderListView::SetDefaultColumnVisibilities()
{
	using namespace DataProviderListView::HeaderDef;
	for (const TPair<FName,FColumnDefinition>& Item : StaticColumnHeader)
	{
		HeaderRow->SetShowGeneratedColumn(Item.Key, Item.Value.bIsVisible);
	}
}

void SDataProviderListView::RefreshMonitorSession(TWeakPtr<IStageMonitorSession> NewSession)
{
	AttachToMonitorSession(NewSession);
	bRebuildListRequested = true;
}

SDataProviderListView::~SDataProviderListView()
{
	if (TSharedPtr<IStageMonitorSession> SessionPtr = Session.Pin())
	{
		SessionPtr->OnStageDataProviderListChanged().RemoveAll(this);
	}
}

void SDataProviderListView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	bool bForceRefresh = false;
	if (bRebuildListRequested)
	{
		RebuildDataProviderList();
		RebuildList();
		bRebuildListRequested = false;
		bForceRefresh = true;
	}

	//Update cached values at specific rate or when list has been rebuilt
	const double RefreshRate = GetDefault<UStageMonitorEditorSettings>()->RefreshRate;
	if (bForceRefresh || (FApp::GetCurrentTime() - LastRefreshTime > RefreshRate))
	{
		LastRefreshTime = FApp::GetCurrentTime();
		for (FDataProviderTableRowDataPtr& RowDataPtr : ListItemsSource)
		{
			RowDataPtr->UpdateCachedValues();
		}
	}

	Super::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

TSharedRef<ITableRow> SDataProviderListView::OnGenerateRow(FDataProviderTableRowDataPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<SDataProviderTableRow> Row = SNew(SDataProviderTableRow, OwnerTable)
		.Padding(2.0f)
		.Item(InItem);

	ListRowWidgets.Add(Row);
	return Row;
}

void SDataProviderListView::OnStageMonitoringMachineListChanged()
{
	bRebuildListRequested = true;
}

void SDataProviderListView::RebuildDataProviderList()
{
	ListItemsSource.Reset();

	if (TSharedPtr<IStageMonitorSession> SessionPtr = Session.Pin())
	{
		const TConstArrayView<FStageSessionProviderEntry> Providers = SessionPtr->GetProviders();
		for (const FStageSessionProviderEntry& Provider : Providers)
		{
			TSharedRef<FDataProviderTableRowData> RowData = MakeShared<FDataProviderTableRowData>(Provider.Identifier, Provider.Descriptor, Session);
			ListItemsSource.Add(RowData);
		}

		for (FDataProviderTableRowDataPtr& TableRowData : ListItemsSource)
		{
			TableRowData->UpdateCachedValues();
		}
	}

	SortProviderList();
	RequestListRefresh();
}

void SDataProviderListView::AttachToMonitorSession(const TWeakPtr<IStageMonitorSession>& NewSession)
{
	if (NewSession != Session)
	{
		if (TSharedPtr<IStageMonitorSession> SessionPtr = Session.Pin())
		{
			SessionPtr->OnStageDataProviderListChanged().RemoveAll(this);
		}

		Session = NewSession;

		if (TSharedPtr<IStageMonitorSession> SessionPtr = Session.Pin())
		{
			SessionPtr->OnStageDataProviderListChanged().AddSP(this, &SDataProviderListView::OnStageMonitoringMachineListChanged);
		}
	}
}

EColumnSortMode::Type SDataProviderListView::GetColumnSortMode(const FName ColumnId) const
{
	if (ColumnId == SortedColumnName)
	{
		return SortMode;
	}

	return EColumnSortMode::None;
}

void SDataProviderListView::OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	SortedColumnName = ColumnId;
	SortMode = InSortMode;

	SortProviderList();
	RequestListRefresh();
}

void SDataProviderListView::SortProviderList()
{
	auto Compare = [this](const FDataProviderTableRowDataPtr& Lhs, const FDataProviderTableRowDataPtr& Rhs, const FName& ColumnName, EColumnSortMode::Type CurrentSortMode)
	{
		if (ColumnName == DataProviderListView::HeaderDef::State)
		{
			return CurrentSortMode == EColumnSortMode::Ascending ? Lhs->CachedState < Rhs->CachedState : Lhs->CachedState > Rhs->CachedState;
		}
		if (ColumnName == DataProviderListView::HeaderDef::Status)
		{
			const int32 CompareResult = Lhs->CachedPerformanceData.Status < Rhs->CachedPerformanceData.Status;
			return CurrentSortMode == EColumnSortMode::Ascending ? CompareResult < 0 : CompareResult > 0;
		}
		else if (ColumnName == DataProviderListView::HeaderDef::MachineName)
		{
			const int32 CompareResult = Lhs->Descriptor.MachineName.Compare(Rhs->Descriptor.MachineName);
			return CurrentSortMode == EColumnSortMode::Ascending ? CompareResult < 0 : CompareResult > 0;
		}
		else if (ColumnName == DataProviderListView::HeaderDef::StageName)
		{
			const int32 CompareResult = Lhs->Descriptor.FriendlyName.Compare(Rhs->Descriptor.FriendlyName);
			return CurrentSortMode == EColumnSortMode::Ascending ? CompareResult < 0 : CompareResult > 0;
		}
		else if (ColumnName == DataProviderListView::HeaderDef::ProcessId)
		{
			return CurrentSortMode == EColumnSortMode::Ascending ? Lhs->Descriptor.ProcessId < Rhs->Descriptor.ProcessId : Lhs->Descriptor.ProcessId > Rhs->Descriptor.ProcessId;
		}
		else if (ColumnName == DataProviderListView::HeaderDef::Roles)
		{
			if (CachedRoleStringToArray.Contains(Lhs->Descriptor.RolesStringified) == false)
			{
				TArray<FString> RoleArray;
				Lhs->Descriptor.RolesStringified.ParseIntoArray(RoleArray, TEXT(","));
				CachedRoleStringToArray.Add(Lhs->Descriptor.RolesStringified, RoleArray);
			}
			if (CachedRoleStringToArray.Contains(Rhs->Descriptor.RolesStringified) == false)
			{
				TArray<FString> RoleArray;
				Rhs->Descriptor.RolesStringified.ParseIntoArray(RoleArray, TEXT(","));
				CachedRoleStringToArray.Add(Rhs->Descriptor.RolesStringified, RoleArray);
			}

			const TArray<FString>& LhsRoles = CachedRoleStringToArray.FindChecked(Lhs->Descriptor.RolesStringified);
			const TArray<FString>& RhsRoles = CachedRoleStringToArray.FindChecked(Rhs->Descriptor.RolesStringified);
			TArray<FString>::TConstIterator LhsIterator = LhsRoles.CreateConstIterator();
			TArray<FString>::TConstIterator RhsIterator = RhsRoles.CreateConstIterator();
			bool bIsValid = false;
			do
			{
				const FString LhsRole = LhsIterator ? *LhsIterator : FString();
				const FString RhsRole = RhsIterator ? *RhsIterator : FString();
				const int32 CompareResult = LhsRole.Compare(RhsRole);
				if (CompareResult != 0)
				{
					return CurrentSortMode == EColumnSortMode::Ascending ? CompareResult < 0 : CompareResult > 0;
				}

				bIsValid = LhsIterator && RhsIterator;
				++LhsIterator;
				++RhsIterator;

			} while (bIsValid);

			return CurrentSortMode == EColumnSortMode::Ascending ? true : false;
		}
		else
		{
			return CurrentSortMode == EColumnSortMode::Ascending ? Lhs->Descriptor.SessionId < Rhs->Descriptor.SessionId : Lhs->Descriptor.SessionId > Rhs->Descriptor.SessionId;
		}
	};

	ListItemsSource.StableSort([&](const FDataProviderTableRowDataPtr& Lhs, const FDataProviderTableRowDataPtr& Rhs)
		{
			return Compare(Lhs, Rhs, SortedColumnName, SortMode);
		});
}

#undef LOCTEXT_NAMESPACE


