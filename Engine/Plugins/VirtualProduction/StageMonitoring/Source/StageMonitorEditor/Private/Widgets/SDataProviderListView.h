// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SListView.h"


struct FDataProviderTableRowData;
class IStageMonitorSession;


using FDataProviderTableRowDataPtr = TSharedPtr<FDataProviderTableRowData>;

/**
 *
 */
class SDataProviderTableRow : public SMultiColumnTableRow<FDataProviderTableRowDataPtr>
{
	using Super = SMultiColumnTableRow<FDataProviderTableRowDataPtr>;

public:
	SLATE_BEGIN_ARGS(SDataProviderTableRow) { }
		SLATE_ARGUMENT(FDataProviderTableRowDataPtr, Item)
		SLATE_ARGUMENT(float, Padding)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwerTableView);


	/** Getters to populate the UI */
	FText GetStateGlyphs() const;
	FSlateColor GetStateColorAndOpacity() const;
	FText GetStatus() const;
	FText GetStatusToolTip() const;
	FText GetTimecode() const;
	FText GetMachineName() const;
	FText GetProcessId() const;
	FText GetStageName() const;
	FText GetRoles() const;
	FText GetAverageFPS() const;
	FText GetEstimatedMaxFPS() const;
	FText GetIdleTime() const;
	FText GetGameThreadTiming() const;
	FText GetGameThreadWaitTiming() const;
	FText GetRenderThreadTiming() const;
	FText GetRenderThreadWaitTiming() const;
	FText GetGPUTiming() const;
	FText GetGPUMem() const;
	FText GetCPUMem() const;
	FText GetAssetsLeftToCompile() const;
	FText GetToolTipText() const;

private:
	/** Handles creation of each columns widget */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	/** Item to display */
	FDataProviderTableRowDataPtr Item;

	/** Last time we refreshed the UI */
	double LastRefreshTime = 0.0;
};


/**
 *
 */
class SDataProviderListView : public SListView<FDataProviderTableRowDataPtr>
{
	using Super = SListView<FDataProviderTableRowDataPtr>;

public:
	SLATE_BEGIN_ARGS(SDataProviderListView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<IStageMonitorSession>& InSession);

	/** Updates the session this widget is currently sourcing data from */
	void RefreshMonitorSession(TWeakPtr<IStageMonitorSession> NewSession);

	/** Cleanup ourselves */
	virtual ~SDataProviderListView();

	/** Used to refresh cached values shown in UI */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	/** Generate a new row for the listview using the Item data */
	TSharedRef<ITableRow> OnGenerateRow(FDataProviderTableRowDataPtr Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Callback triggered when provider list changed */
	void OnStageMonitoringMachineListChanged();

	/** Rebuild provider list from scratch */
	void RebuildDataProviderList();

	/** Binds to the current session desired delegates */
	void AttachToMonitorSession(const TWeakPtr<IStageMonitorSession>& NewSession);

	/** Returns sort method for a given column name */
	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;

	/** Called when sort mode changed for a given column */
	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

	/** Sorts provider list based on active column sort mode */
	void SortProviderList();

private:

	/** Set the default column visibility state. */
	void SetDefaultColumnVisibilities();

	/** Widget and data containing info about what's shown in the listview */
	TArray<FDataProviderTableRowDataPtr> ListItemsSource;
	TArray<TWeakPtr<SDataProviderTableRow>> ListRowWidgets;

	/** Pointer to the stage session data */
	TWeakPtr<IStageMonitorSession> Session;

	/** Used to cache if list needs to be refreshed or not */
	bool bRebuildListRequested = false;

	/** Timestamp when we last refreshed UI */
	double LastRefreshTime = 0.0;

	/** Currently sorted column name */
	FName SortedColumnName;

	/** Sort mode for the currently sorted column */
	EColumnSortMode::Type SortMode = EColumnSortMode::Ascending;

	/** Cache of stringified roles to array of roles to avoid parsing into array constantly */
	TMap<FString, TArray<FString>> CachedRoleStringToArray;

	/** Boolean guard to prevent reentrant updates to the column visibility state. */
	bool bUpdatingColumnVisibility = false;
};

