// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class StageMonitorEditor : ModuleRules
{
	public StageMonitorEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
			new string[]
			{
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"DesktopPlatform",
				"EditorStyle",
				"Engine",
				"GameplayTags",
 				"InputCore",
				"Json",
				"LevelEditor",
				"Projects",
				"Slate",
				"SlateCore",
				"StageDataCore",
				"StageMonitor",
				"StageMonitorCommon",
				"UnrealEd",
                "WorkspaceMenuStructure",
            }
		);
	}
}
