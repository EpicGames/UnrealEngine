{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Stage Monitor",
	"Description": "Plugin enabling monitoring in the context of a virtual production stage where multiple machines are in operation",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Plugins": [
		{
			"Name": "Takes",
			"Enabled": true
		},
		{
			"Name": "VirtualProductionUtilities",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "StageMonitor",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "StageMonitorEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "StageDataProvider",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "StageMonitorCommon",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}
